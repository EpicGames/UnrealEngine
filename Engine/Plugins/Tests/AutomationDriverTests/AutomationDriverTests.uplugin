{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Automation Driver Tests",
	"Description": "",
	"Category": "Testing",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "AutomationDriverTests",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault"
		}
	]
}