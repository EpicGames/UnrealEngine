// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AutomationDriverTests : ModuleRules
{
	public AutomationDriverTests(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"InputCore",
				"AutomationDriver",
				
				// ... add other public dependencies that you statically link with here ...
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"AutomationDriver",

				"Core",
				"ApplicationCore",
				"Json",

				// ... add private dependencies that you statically link with here ...	
			}
			);
	}
}
