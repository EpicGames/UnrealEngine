// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "DriverConfiguration.h"
#include "SAutomationDriverSpecSuite.h"
#include "AutomationDriverSpecSuiteViewModel.h"
#include "Framework/Application/SlateApplication.h"
#include "IAutomationDriver.h"
#include "IAutomationDriverModule.h"
#include "IDriverElement.h"
#include "IDriverSequence.h"
#include "Layout/WidgetPath.h"
#include "LocateBy.h"
#include "Widgets/SWindow.h"

#define TEST_TRUE(expression) \
	EPIC_TEST_BOOLEAN_(TEXT(#expression), expression, true)

#define TEST_FALSE(expression) \
	EPIC_TEST_BOOLEAN_(TEXT(#expression), expression, false)

#define TEST_EQUAL(expression, expected) \
	EPIC_TEST_BOOLEAN_(TEXT(#expression), expression, expected)

#define EPIC_TEST_BOOLEAN_(text, expression, expected) \
	TestEqual(text, expression, expected);
	  
BEGIN_DEFINE_SPEC(FAutomationDriverSpec, "TestFramework.Driver", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
	TSharedPtr<SWindow> SuiteWindow;
	TSharedPtr<SAutomationDriverSpecSuite> SuiteWidget;
	TSharedPtr<IAutomationDriverSpecSuiteViewModel> SuiteViewModel;
	FAutomationDriverPtr Driver;
END_DEFINE_SPEC(FAutomationDriverSpec)
void FAutomationDriverSpec::Define()
{
	BeforeEach([this]() {
		if (IAutomationDriverModule::Get().IsEnabled())
		{
			IAutomationDriverModule::Get().Disable();
		}

		IAutomationDriverModule::Get().Enable();
		
		if (!SuiteViewModel.IsValid())
		{
			SuiteViewModel = FSpecSuiteViewModelFactory::Create();
		}

		if (!SuiteWidget.IsValid())
		{
			SuiteWidget = SNew(SAutomationDriverSpecSuite, SuiteViewModel.ToSharedRef());
		}

		if (!SuiteWindow.IsValid())
		{
			SuiteWindow = FSlateApplication::Get().AddWindow(
				SNew(SWindow)
				.Title(FText::FromString(TEXT("Automation Driver Spec Suite")))
				.HasCloseButton(true)
				.SupportsMaximize(true)
				.SupportsMinimize(true)
				.ClientSize(FVector2D(600, 540))
				[
					SuiteWidget.ToSharedRef()
				]);
			// Clean up Share pointer if the window is closed. Because if the window is closed, it can not be brought back to front.
			SuiteWindow->SetOnWindowClosed(FOnWindowClosed::CreateLambda([this](const TSharedRef<SWindow>& Window){ SuiteWindow.Reset(); }));
		}

		SuiteWidget->RestoreContents();
		SuiteWindow->BringToFront(true);
		FSlateApplication::Get().SetKeyboardFocus(SuiteWindow, EFocusCause::SetDirectly);
		SuiteViewModel->Reset();

		Driver = IAutomationDriverModule::Get().CreateDriver();
	});

	Describe("FindElement", [this]()
	{
		It("should fail to locate a element when more than one element is located", EAsyncExecution::ThreadPool, [this]()
		{
			TEST_FALSE(Driver->FindElement(By::Id("Duplicate"))->Exists());
		});

		It("should fail to locate a SWidget if none exist with the specified Id", EAsyncExecution::ThreadPool, [this]()
		{
			TEST_FALSE(Driver->FindElement(By::Id("NotDefined"))->Exists());
		});

		Describe("By::Id", [this]()
		{
			It("should locate a SWidget with the specified Id", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_TRUE(Driver->FindElement(By::Id("KeyA"))->Exists());
			});
		});

		Describe("By::Path", [this]()
		{
			It("should be capable of locating a SWidget by its Tag", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_TRUE(Driver->FindElement(By::Path("Keyboard"))->Exists());
				TEST_TRUE(Driver->FindElement(By::Path("List"))->Exists());
				TEST_TRUE(Driver->FindElement(By::Path("Tiles"))->Exists());
			});

			It("should be capable of locating a SWidget by a hierarchy of Tags", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_TRUE(Driver->FindElement(By::Path("Documents//List"))->Exists());
				TEST_TRUE(Driver->FindElement(By::Path("Documents//Tiles"))->Exists());
			});

			It("should be capable of locating a SWidget by its Id", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_TRUE(Driver->FindElement(By::Path("#Suite"))->Exists());
				TEST_TRUE(Driver->FindElement(By::Path("#Piano"))->Exists());
				TEST_TRUE(Driver->FindElement(By::Path("#KeyA"))->Exists());
			});

			It("should be capable of locating a SWidget by a hierarchy of Ids", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_TRUE(Driver->FindElement(By::Path("#Suite//#Piano"))->Exists());
				TEST_TRUE(Driver->FindElement(By::Path("#Suite//#KeyA"))->Exists());
				TEST_TRUE(Driver->FindElement(By::Path("#Suite//#Piano//#KeyA"))->Exists());
			});

			It("should be capable of locating a SWidget by its Type", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_TRUE(Driver->FindElement(By::Path("<SAutomationDriverSpecSuite>"))->Exists());
			});

			It("should be capable of locating a SWidget by a hierarchy of Types", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_TRUE(Driver->FindElement(By::Path("<SWindow>//<SAutomationDriverSpecSuite>"))->Exists());
			});

			It("should be capable of locating a SWidget by a mixed hierarchy of Ids and Types", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_TRUE(Driver->FindElement(By::Path("#Piano//#KeyB/<STextBlock>"))->Exists());
				TEST_TRUE(Driver->FindElement(By::Path("#Piano//#KeyD/<STextBlock>"))->Exists());
			});

			It("should be capable of locating a SWidget by a hierarchy of Tags and Ids", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_TRUE(Driver->FindElement(By::Path("#Documents//List"))->Exists());
				TEST_TRUE(Driver->FindElement(By::Path("#Documents//Tiles"))->Exists());
				TEST_TRUE(Driver->FindElement(By::Path("Form//Rows//#A1"))->Exists());
				TEST_TRUE(Driver->FindElement(By::Path("Form//Rows//#C1"))->Exists());
			});

			It("should be capable of locating a SWidget by a hierarchy of Types and Tags", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_TRUE(Driver->FindElement(By::Path("Form//<SMultiLineEditableTextBox>"))->Exists());
				TEST_TRUE(Driver->FindElement(By::Path("Form//Rows//#A1//<SEditableText>"))->Exists());
			});

			It("should be capable of locating a SWidget by a hierarchy of Types, Tags and Ids", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_TRUE(Driver->FindElement(By::Path("#Suite//Form//Rows//#A1//<SEditableText>"))->Exists());
				TEST_TRUE(Driver->FindElement(By::Path("#Suite//Keyboard//#KeyE/<STextBlock>"))->Exists());
				TEST_TRUE(Driver->FindElement(By::Path("#Suite//<SVerticalBox>//#KeyE/<STextBlock>"))->Exists());
			});

			It("should be capable of locating direct SWidget child descendants", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_TRUE(Driver->FindElement(By::Path("#KeyE/<STextBlock>"))->Exists());
				TEST_TRUE(Driver->FindElement(By::Path("<SWindow>/<SOverlay>/<SVerticalBox>/<SVerticalBox>/<SAutomationDriverSpecSuite>"))->Exists());
			});

			It("should be capable of locating indirect SWidget child descendants", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_TRUE(Driver->FindElement(By::Path("Form//<SMultiLineEditableTextBox>"))->Exists());
				TEST_TRUE(Driver->FindElement(By::Path("#Suite//#KeyA"))->Exists());
			});

		});
	});

	Describe("FindElements", [this]()
	{
		It("should fail to locate any elements if none exist with the specified Id", EAsyncExecution::ThreadPool, [this]()
		{
			TEST_EQUAL(Driver->FindElements(By::Id("NotDefined"))->GetElements().Num(), 0);
		});

		Describe("By::Id", [this]()
		{
			It("should locate SWidgets with the specified Id", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_EQUAL(Driver->FindElements(By::Id("KeyD"))->GetElements().Num(), 1);
				TEST_EQUAL(Driver->FindElements(By::Id("DuplicateId"))->GetElements().Num(), 2);
			});
		});

		Describe("By::Path", [this]()
		{
			It("should be capable of locating SWidgets by Tag", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_EQUAL(Driver->FindElements(By::Path("Keyboard"))->GetElements().Num(), 1);
				TEST_EQUAL(Driver->FindElements(By::Path("Key"))->GetElements().Num(), 7);
				TEST_EQUAL(Driver->FindElements(By::Path("TextBox"))->GetElements().Num(), 7);
			});

			It("should be capable of locating SWidgets by a hierarchy of Tags", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_EQUAL(Driver->FindElements(By::Path("Keyboard//Key"))->GetElements().Num(), 7);
				TEST_EQUAL(Driver->FindElements(By::Path("Form//TextBox"))->GetElements().Num(), 7);
			});

			It("should be capable of locating SWidgets by Id", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_EQUAL(Driver->FindElements(By::Path("#Suite"))->GetElements().Num(), 1);
				TEST_EQUAL(Driver->FindElements(By::Path("#Piano"))->GetElements().Num(), 1);
				TEST_EQUAL(Driver->FindElements(By::Path("#KeyA"))->GetElements().Num(), 1);
			});

			It("should be capable of locating SWidgets by a hierarchy of Ids", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_EQUAL(Driver->FindElements(By::Path("#Suite//#Piano"))->GetElements().Num(), 1);
				TEST_EQUAL(Driver->FindElements(By::Path("#Suite//#KeyA"))->GetElements().Num(), 1);
				TEST_EQUAL(Driver->FindElements(By::Path("#Suite//#Piano//#KeyA"))->GetElements().Num(), 1);
			});

			It("should be capable of locating SWidgets by Type", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_EQUAL(Driver->FindElements(By::Path("<SAutomationDriverSpecSuite>"))->GetElements().Num(), 1);
			});

			It("should be capable of locating SWidgets by a hierarchy of Types", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_EQUAL(Driver->FindElements(By::Path("<SWindow>//<SAutomationDriverSpecSuite>"))->GetElements().Num(), 1);
				TEST_EQUAL(Driver->FindElements(By::Path("<SAutomationDriverSpecSuite>/<SVerticalBox>"))->GetElements().Num(), 1);
			});

			It("should be capable of locating SWidgets by a mixed hierarchy of Ids and Types", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_EQUAL(Driver->FindElements(By::Path("#Piano//<STextBlock>"))->GetElements().Num(), 14);
				TEST_EQUAL(Driver->FindElements(By::Path("#Piano//#KeyD/<STextBlock>"))->GetElements().Num(), 1);
			});

			It("should be capable of locating SWidgets by a hierarchy of Tags and Ids", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_EQUAL(Driver->FindElements(By::Path("#Documents//List"))->GetElements().Num(), 1);
				TEST_EQUAL(Driver->FindElements(By::Path("Form//Rows//#A1"))->GetElements().Num(), 1);
				TEST_EQUAL(Driver->FindElements(By::Path("#UserForm//Rows"))->GetElements().Num(), 3);
			});

			It("should be capable of locating SWidgets by a hierarchy of Types and Tags", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_EQUAL(Driver->FindElements(By::Path("Form//Rows//#A1//<SEditableText>"))->GetElements().Num(), 1);
				TEST_EQUAL(Driver->FindElements(By::Path("Form//Rows//<SEditableText>"))->GetElements().Num(), 6);
			});

			It("should be capable of locating SWidgets by a hierarchy of Types, Tags and Ids", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_EQUAL(Driver->FindElements(By::Path("#Documents//<SScrollBox>//Document"))->GetElements().Num(), 200);
				TEST_EQUAL(Driver->FindElements(By::Path("#UserForm//Rows//<SEditableText>"))->GetElements().Num(), 6);
				TEST_EQUAL(Driver->FindElements(By::Path("#Suite//Form//Rows//#A1//<SEditableText>"))->GetElements().Num(), 1);
				TEST_EQUAL(Driver->FindElements(By::Path("#Suite//<SVerticalBox>//Key/<STextBlock>"))->GetElements().Num(), 7);
				TEST_EQUAL(Driver->FindElements(By::Path("#Suite//Keyboard//#KeyE/<STextBlock>"))->GetElements().Num(), 1);
			});

			It("should be capable of locating only descendants of a specific set of elements", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverElementRef UserForm = Driver->FindElement(By::Id("UserForm"));
				TEST_TRUE(UserForm->Exists());
				TEST_EQUAL(Driver->FindElements(By::Path(UserForm, "Rows"))->GetElements().Num(), 3);
			});
		});
	});

	Describe("By::TextFilter", [this]()
	{
		Describe("Equals", [this]()
		{
			static const FString ElementText = "Bb/A#";
			static const FString ElementSubstringText = ElementText.LeftChop(1);
			static const FString ElementWrongCaseText = ElementText.ToLower();

			It("should fail to locate an element if none exist with the specified text", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_FALSE(Driver->FindElement(By::TextFilter::Equals(By::Path("#Piano//<SButton>"), "NotDefined"))->Exists());
			});

			It("should fail to locate an element containing the specified substring in the text", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_FALSE(Driver->FindElement(By::TextFilter::Equals(By::Path("#Piano//<SButton>"), ElementSubstringText))->Exists());
			});

			Describe("IgnoreCase", [this]()
			{
				It("should locate an element with the specified text in a different case", EAsyncExecution::ThreadPool, [this]()
				{
					TEST_TRUE(Driver->FindElement(By::TextFilter::Equals(By::Path("#Piano//<SButton>"), ElementWrongCaseText, ESearchCase::IgnoreCase))->Exists());
				});

				It("should locate an element with the specified text in the proper case", EAsyncExecution::ThreadPool, [this]()
				{
					TEST_TRUE(Driver->FindElement(By::TextFilter::Equals(By::Path("#Piano//<SButton>"), ElementText, ESearchCase::IgnoreCase))->Exists());
				});
			});

			Describe("CaseSensitive", [this]()
			{
				It("should fail to locate an element with the specified text in a different case", EAsyncExecution::ThreadPool, [this]()
				{
					TEST_FALSE(Driver->FindElement(By::TextFilter::Equals(By::Path("#Piano//<SButton>"), ElementWrongCaseText, ESearchCase::CaseSensitive))->Exists());
				});

				It("should locate an element with the specified text in the proper case", EAsyncExecution::ThreadPool, [this]()
				{
					TEST_TRUE(Driver->FindElement(By::TextFilter::Equals(By::Path("#Piano//<SButton>"), ElementText, ESearchCase::CaseSensitive))->Exists());
				});
			});

			Describe("Default", [this]()
			{
				It("should use case sensitive comparison by default", EAsyncExecution::ThreadPool, [this]()
				{
					TEST_TRUE(Driver->FindElement(By::TextFilter::Equals(By::Path("#Piano//<SButton>"), ElementText))->Exists());
					TEST_FALSE(Driver->FindElement(By::TextFilter::Equals(By::Path("#Piano//<SButton>"), ElementWrongCaseText))->Exists());
				});
			});
			
			It("should be capable of locating multiple elements", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_EQUAL(Driver->FindElements(By::TextFilter::Equals(By::Path("Documents//<SButton>"), "Document 1", ESearchCase::CaseSensitive))->GetElements().Num(), 3);
			});
		});
		
		Describe("Contains", [this]()
		{
			static const FString ElementText = "Bb/A#";
			static const FString ElementSubstringText = ElementText.LeftChop(1);
			static const FString ElementWrongCaseSubstringText = ElementSubstringText.ToLower();

			static const FString ElementLeftSubstringText = ElementText.LeftChop(1);
			static const FString ElementRightSubstringText = ElementText.RightChop(1);
			static const FString ElementCenterSubstringText = ElementText.LeftChop(1).RightChop(1);

			It("should fail to locate an element if none exist with the specified text", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_FALSE(Driver->FindElement(By::TextFilter::Contains(By::Path("#Piano//<SButton>"), "NotDefined"))->Exists());
			});

			It("should locate an element containing the specified substring at any position in the text", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_TRUE(Driver->FindElement(By::TextFilter::Contains(By::Path("#Piano//<SButton>"), ElementLeftSubstringText))->Exists());
				TEST_TRUE(Driver->FindElement(By::TextFilter::Contains(By::Path("#Piano//<SButton>"), ElementCenterSubstringText))->Exists());
				TEST_TRUE(Driver->FindElement(By::TextFilter::Contains(By::Path("#Piano//<SButton>"), ElementRightSubstringText))->Exists());
			});

			Describe("IgnoreCase", [this]()
			{
				It("should locate an element containing the specified substring in a different case", EAsyncExecution::ThreadPool, [this]()
				{
					TEST_TRUE(Driver->FindElement(By::TextFilter::Contains(By::Path("#Piano//<SButton>"), ElementWrongCaseSubstringText, ESearchCase::IgnoreCase))->Exists());
				});

				It("should locate an element containing the specified substring in the proper case", EAsyncExecution::ThreadPool, [this]()
				{
					TEST_TRUE(Driver->FindElement(By::TextFilter::Contains(By::Path("#Piano//<SButton>"), ElementSubstringText, ESearchCase::IgnoreCase))->Exists());
				});
			});

			Describe("CaseSensitive", [this]()
			{
				It("should fail to locate an element containing the specified substring in a different case", EAsyncExecution::ThreadPool, [this]()
				{
					TEST_FALSE(Driver->FindElement(By::TextFilter::Contains(By::Path("#Piano//<SButton>"), ElementWrongCaseSubstringText, ESearchCase::CaseSensitive))->Exists());
				});

				It("should locate an element containing the specified substring in the proper case", EAsyncExecution::ThreadPool, [this]()
				{
					TEST_TRUE(Driver->FindElement(By::TextFilter::Contains(By::Path("#Piano//<SButton>"), ElementSubstringText, ESearchCase::CaseSensitive))->Exists());
				});
			});

			Describe("SearchDir", [this]()
			{
				It("should locate an element containing the specified substring by searching in both directions", EAsyncExecution::ThreadPool, [this]()
				{
					TEST_TRUE(Driver->FindElement(By::TextFilter::Contains(By::Path("#Piano//<SButton>"), ElementLeftSubstringText, ESearchCase::CaseSensitive, ESearchDir::FromStart))->Exists());
					TEST_TRUE(Driver->FindElement(By::TextFilter::Contains(By::Path("#Piano//<SButton>"), ElementLeftSubstringText, ESearchCase::CaseSensitive, ESearchDir::FromEnd))->Exists());
					TEST_TRUE(Driver->FindElement(By::TextFilter::Contains(By::Path("#Piano//<SButton>"), ElementRightSubstringText, ESearchCase::CaseSensitive, ESearchDir::FromStart))->Exists());
					TEST_TRUE(Driver->FindElement(By::TextFilter::Contains(By::Path("#Piano//<SButton>"), ElementRightSubstringText, ESearchCase::CaseSensitive, ESearchDir::FromEnd))->Exists());
				});
			});

			Describe("Default", [this]()
			{
				It("should ignore case by default", EAsyncExecution::ThreadPool, [this]()
				{
					TEST_TRUE(Driver->FindElement(By::TextFilter::Contains(By::Path("#Piano//<SButton>"), ElementSubstringText))->Exists());
					TEST_TRUE(Driver->FindElement(By::TextFilter::Contains(By::Path("#Piano//<SButton>"), ElementWrongCaseSubstringText))->Exists());
				});
			});
			
			It("should be capable of locating multiple elements", EAsyncExecution::ThreadPool, [this]()
			{
				const FString ElementsSubstringText = "ocument 2";
				const int32 ElementsNum = 12; // "Document 2", "Document 20", ..., "Document 29", "Document 200"
				TEST_EQUAL(Driver->FindElements(By::TextFilter::Contains(By::Path("#Documents//<SScrollBox>//<SButton>"), ElementsSubstringText, ESearchCase::CaseSensitive))->GetElements().Num(), ElementsNum);
			});
		});
	});

	Describe("Element", [this]()
	{
		Describe("Hover", [this]()
		{
			It("should move the cursor over the element", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_TRUE(Driver->FindElement(By::Id("KeyE"))->Hover());
			});

			Describe("attempt to scroll the element into view (Scrolling Down) if it exists but isn't visible and then move the cursor over the element", [this]()
			{
				BeforeEach([this]() {
					SuiteWidget->ScrollDocumentsToTop();
				});

				It("", EAsyncExecution::ThreadPool, [this]()
				{
					FDriverElementRef ScrollBox = Driver->FindElement(By::Path("#Documents//<SScrollBox>"));
					Driver->Wait(Until::ElementIsScrolledToBeginning(ScrollBox, FWaitTimeout::InSeconds(3)));

					TEST_TRUE(Driver->FindElement(By::Path("#Documents//<SScrollBox>//#Document150"))->Hover());
				});
			});

			Describe("attempt to scroll the element into view (Scrolling Up) if it exists but isn't visible and then move the cursor over the element", [this]()
			{
				BeforeEach([this]() {
					SuiteWidget->ScrollDocumentsToBottom();
				});

				It("", EAsyncExecution::ThreadPool, [this]()
				{
					FDriverElementRef ScrollBox = Driver->FindElement(By::Path("#Documents//<SScrollBox>"));
					Driver->Wait(Until::ElementIsScrolledToEnd(ScrollBox, FWaitTimeout::InSeconds(3)));

					TEST_TRUE(Driver->FindElement(By::Path("#Documents//<SScrollBox>//#Document50"))->Hover());
				});
			});
		});

		Describe("IsHovered", [this]()
		{
			It("should return true if the element is a SWidget currently under the cursor", EAsyncExecution::ThreadPool, [this]()
			{
				Driver->FindElement(By::Id("KeyC"))->Hover();
				TEST_TRUE(Driver->FindElement(By::Id("KeyC"))->IsHovered());
			});

			It("should return false if the element is a SWidget not currently under the cursor", EAsyncExecution::ThreadPool, [this]()
			{
				Driver->FindElement(By::Id("KeyB"))->Hover();
				TEST_FALSE(Driver->FindElement(By::Id("KeyC"))->IsHovered());
			});

			It("should return false if the element can not be found", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_FALSE(Driver->FindElement(By::Id("NotDefined"))->IsHovered());
			});
		});

		Describe("Click", [this]()
		{
			It("should simulate a cursor move and click on a valid tagged widget", EAsyncExecution::ThreadPool, [this]()
			{
				Driver->FindElement(By::Id("KeyG"))->Click();
				TEST_EQUAL(SuiteViewModel->GetKeySequence(), TEXT("G"));
			});

			It("should simulate a cursor move and click on multiple valid tagged widgets", EAsyncExecution::ThreadPool, [this]()
			{
				Driver->FindElement(By::Id("KeyA"))->Click();
				Driver->FindElement(By::Id("KeyB"))->Click();
				Driver->FindElement(By::Id("KeyC"))->Click();

				TEST_EQUAL(SuiteViewModel->GetKeySequence(), TEXT("ABC"));
			});

			It("should simulate a cursor move and click on multiple valid tagged widgets, in sequence", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverElementRef KeyA = Driver->FindElement(By::Id("KeyA"));
				FDriverElementRef KeyB = Driver->FindElement(By::Id("KeyB"));

				FDriverSequenceRef Sequence = Driver->CreateSequence();
				Sequence->Actions()
					.Click(By::Id("KeyC"))
					.Click(KeyB)
					.Click(KeyA);

				TEST_TRUE(Sequence->Perform());
				TEST_EQUAL(SuiteViewModel->GetKeySequence(), TEXT("CBA"));
			});

			It("should simulate a cursor move and click and wait for the element to become interactable", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverElementRef KeyE = Driver->FindElement(By::Id("KeyE"));

				SuiteViewModel->SetKeyResetDelay(FTimespan::FromSeconds(1.5));
				KeyE->Click();
				Driver->Wait(FTimespan::FromSeconds(2));
				KeyE->Click();
				SuiteViewModel->SetKeyResetDelay(FTimespan::Zero());

				TEST_EQUAL(SuiteViewModel->GetKeySequence(), TEXT("EE"));
			});

			It("should simulate a cursor move and click and wait for the element to become visible", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverElementRef KeyC = Driver->FindElement(By::Id("KeyC"));

				SuiteViewModel->SetPianoVisibility(EVisibility::Collapsed);

				TSharedRef<int32, ESPMode::ThreadSafe> OriginalExistenceChecker = MakeShareable(new int32);
				TWeakPtr<int32, ESPMode::ThreadSafe> WeakExistenceChecker = OriginalExistenceChecker;
				AsyncTask(
					ENamedThreads::GameThread,
					[this, WeakExistenceChecker]()
					{
						const TSharedPtr<int32, ESPMode::ThreadSafe> ExistenceChecker = WeakExistenceChecker.Pin();
						if (ExistenceChecker.IsValid())
						{
							FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this, WeakExistenceChecker](float Delta){
								const TSharedPtr<int32, ESPMode::ThreadSafe> TickerExistenceChecker = WeakExistenceChecker.Pin();
								if (TickerExistenceChecker.IsValid())
								{
									SuiteViewModel->SetPianoVisibility(EVisibility::Visible);
								}
								return false;
							}), 1.5);
						}
					}
				);
				KeyC->Click();
				TEST_EQUAL(SuiteViewModel->GetKeySequence(), TEXT("C"));
			});
		});

		Describe("DoubleClick", [this]()
		{
			It("should simulate a cursor move and double click on a valid tagged widget", EAsyncExecution::ThreadPool, [this]()
			{
				Driver->FindElement(By::Id("KeyB"))->DoubleClick();
				TEST_EQUAL(SuiteViewModel->GetKeySequence(), TEXT("BB"));
			});

			It("should simulate a cursor move and click on multiple valid tagged widgets", EAsyncExecution::ThreadPool, [this]()
			{
				Driver->FindElement(By::Id("KeyA"))->DoubleClick();
				Driver->FindElement(By::Id("KeyB"))->DoubleClick();
				Driver->FindElement(By::Id("KeyC"))->DoubleClick();
				TEST_EQUAL(SuiteViewModel->GetKeySequence(), TEXT("AABBCC"));
			});

			It("should simulate a cursor move and click on multiple valid tagged widgets, in sequence", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverElementRef KeyA = Driver->FindElement(By::Id("KeyA"));
				FDriverElementRef KeyB = Driver->FindElement(By::Id("KeyB"));

				FDriverSequenceRef Sequence = Driver->CreateSequence();
				Sequence->Actions()
					.DoubleClick(By::Id("KeyC"))
					.DoubleClick(KeyB)
					.DoubleClick(KeyA);

				TEST_TRUE(Sequence->Perform());
				TEST_EQUAL(SuiteViewModel->GetKeySequence(), TEXT("CCBBAA"));
			});

			It("should simulate a cursor move and click and wait for the element to become interactable", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverElementRef KeyE = Driver->FindElement(By::Id("KeyE"));

				SuiteViewModel->SetKeyResetDelay(FTimespan::FromSeconds(1.5));
				KeyE->DoubleClick();
				KeyE->DoubleClick();
				SuiteViewModel->SetKeyResetDelay(FTimespan::Zero());

				TEST_EQUAL(SuiteViewModel->GetKeySequence(), TEXT("EE"));
			});

			It("should simulate a cursor move and click and wait for the element to become visible", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverElementRef KeyC = Driver->FindElement(By::Id("KeyC"));

				SuiteViewModel->SetPianoVisibility(EVisibility::Collapsed);

				TSharedRef<int32, ESPMode::ThreadSafe> OriginalExistenceChecker = MakeShareable(new int32);
				TWeakPtr<int32, ESPMode::ThreadSafe> WeakExistenceChecker = OriginalExistenceChecker;
				AsyncTask(
					ENamedThreads::GameThread,
					[this, WeakExistenceChecker]()
					{
						const TSharedPtr<int32, ESPMode::ThreadSafe> ExistenceChecker = WeakExistenceChecker.Pin();
						if (ExistenceChecker.IsValid())
						{
							FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this, WeakExistenceChecker](float Delta){
								const TSharedPtr<int32, ESPMode::ThreadSafe> TickerExistenceChecker = WeakExistenceChecker.Pin();
								if (TickerExistenceChecker.IsValid())
								{
									SuiteViewModel->SetPianoVisibility(EVisibility::Visible);
								}
								return false;
							}), 1.5);
						}
					}
				);
				KeyC->DoubleClick();
				TEST_EQUAL(SuiteViewModel->GetKeySequence(), TEXT("CC"));
			});
		});

		Describe("ScrollBy", [this]()
		{
			Describe("should simulate a mouse wheel event at the current cursor position by the specified negative delta", [this]()
			{
				BeforeEach([this]() {
					SuiteWidget->ScrollDocumentsToTop();
				});

				It("", EAsyncExecution::ThreadPool, [this]()
				{
					FDriverElementRef Element = Driver->FindElement(By::Path("#Documents//Tiles"));
					Driver->Wait(Until::ElementIsScrolledToBeginning(Element, FWaitTimeout::InSeconds(3)));

					TEST_TRUE(Element->IsScrolledToBeginning());
					TEST_TRUE(Element->ScrollBy(-1));
					TEST_FALSE(Element->IsScrolledToBeginning());
				});
			});

			Describe("should simulate a mouse wheel event at the current cursor position by the specified positive delta", [this]()
			{
				BeforeEach([this]() {
					SuiteWidget->ScrollDocumentsToBottom();
				});

				It("", EAsyncExecution::ThreadPool, [this]()
				{
					FDriverElementRef Element = Driver->FindElement(By::Path("#Documents//List"));
					Driver->Wait(Until::ElementIsScrolledToEnd(Element, FWaitTimeout::InSeconds(3)));

					TEST_TRUE(Element->IsScrolledToEnd());
					TEST_TRUE(Element->ScrollBy(1));
					TEST_FALSE(Element->IsScrolledToEnd());
				});
			});
		});

		Describe("ScrollToBeginning", [this]()
		{
			Describe("should move the cursor over the element and simulate a positive delta mouse wheel event by the specified amount", [this]()
			{
				BeforeEach([this]() {
					SuiteWidget->ScrollDocumentsToBottom();
				});

				It("", EAsyncExecution::ThreadPool, [this]()
				{
					FDriverElementRef Element = Driver->FindElement(By::Path("#Documents//List"));
					Driver->Wait(Until::ElementIsScrolledToEnd(Element, FWaitTimeout::InSeconds(3)));

					TEST_TRUE(Element->IsScrolledToEnd());
					TEST_TRUE(Element->ScrollToBeginning(1));
					TEST_TRUE(Element->IsScrolledToBeginning());
				});
			});

			Describe("should move the cursor over the element and simulate positive delta mouse wheel events until the beginning is reached", [this]()
			{
				BeforeEach([this]() {
					SuiteWidget->ScrollDocumentsToBottom();
				});

				It("", EAsyncExecution::ThreadPool, [this]()
				{
					FDriverElementRef Element = Driver->FindElement(By::Path("#Documents//Tiles"));
					Driver->Wait(Until::ElementIsScrolledToEnd(Element, FWaitTimeout::InSeconds(3)));

					TEST_TRUE(Element->IsScrolledToEnd());
					TEST_TRUE(Element->ScrollToBeginning());
					TEST_TRUE(Element->IsScrolledToBeginning());
				});
			});
		});

		Describe("ScrollToBeginningUntil", [this]()
		{
			Describe("should move the cursor over the element and simulate positive mouse wheel events until the specified sub-element is visible or the beginning is reached", [this]()
			{
				BeforeEach([this]() {
					SuiteWidget->ScrollDocumentsToBottom();
				});

				It("", EAsyncExecution::ThreadPool, [this]()
				{
					FDriverElementRef Element = Driver->FindElement(By::Path("#Documents//Tiles"));
					Driver->Wait(Until::ElementIsScrolledToEnd(Element, FWaitTimeout::InSeconds(3)));
					TEST_TRUE(Element->IsScrolledToEnd());

					FDriverElementRef Document = Driver->FindElement(By::Path("#Documents//Tiles//#Document150"));
					TEST_FALSE(Document->Exists());

					TEST_TRUE(Element->ScrollToBeginningUntil(Document));
					TEST_TRUE(Document->IsVisible());
				});
			});

			Describe("should fail if scrollable element reaches the beginning and the desired element hasn't been found", [this]()
			{
				BeforeEach([this]() {
					SuiteWidget->ScrollDocumentsToBottom();
				});

				It("", EAsyncExecution::ThreadPool, [this]()
				{
					FDriverElementRef Element = Driver->FindElement(By::Path("#Documents//List"));
					Driver->Wait(Until::ElementIsScrolledToEnd(Element, FWaitTimeout::InSeconds(3)));
					TEST_TRUE(Element->IsScrolledToEnd());

					FDriverElementRef Document = Driver->FindElement(By::Id("NotDefined"));
					TEST_FALSE(Element->ScrollToBeginningUntil(Document));
				});
			});
		});

		Describe("ScrollToEnd", [this]()
		{
			Describe("should move the cursor over the element and simulate a negative delta mouse wheel event by the specified amount", [this]()
			{
				BeforeEach([this]() {
					SuiteWidget->ScrollDocumentsToTop();
				});

				It("", EAsyncExecution::ThreadPool, [this]()
				{
					FDriverElementRef Element = Driver->FindElement(By::Path("#Documents//Tiles"));
					Driver->Wait(Until::ElementIsScrolledToBeginning(Element, FWaitTimeout::InSeconds(3)));

					TEST_TRUE(Element->IsScrolledToBeginning());
					TEST_TRUE(Element->ScrollToEnd(1));
					TEST_TRUE(Element->IsScrolledToEnd());
				});
			});

			Describe("should move the cursor over the element and simulate negative delta mouse wheel events until the end is reached", [this]()
			{
				BeforeEach([this]() {
					SuiteWidget->ScrollDocumentsToTop();
				});

				It("", EAsyncExecution::ThreadPool, [this]()
				{
					FDriverElementRef Element = Driver->FindElement(By::Path("#Documents//List"));
					Driver->Wait(Until::ElementIsScrolledToBeginning(Element, FWaitTimeout::InSeconds(3)));

					TEST_TRUE(Element->IsScrolledToBeginning());
					TEST_TRUE(Element->ScrollToEnd());
					TEST_TRUE(Element->IsScrolledToEnd());
				});
			});
		});

		Describe("ScrollToEndUntil", [this]()
		{
			Describe("should move the cursor over the element and simulate a mouse wheel events until the specified sub-element is visible", [this]()
			{
				BeforeEach([this]() {
					SuiteWidget->ScrollDocumentsToTop();
				});

				It("", EAsyncExecution::ThreadPool, [this]()
				{
					FDriverElementRef Element = Driver->FindElement(By::Path("#Documents//List"));
					Driver->Wait(Until::ElementIsScrolledToBeginning(Element, FWaitTimeout::InSeconds(3)));
					TEST_TRUE(Element->IsScrolledToBeginning());

					FDriverElementRef Document = Driver->FindElement(By::Path("#Documents//List//#Document50"));
					TEST_FALSE(Document->Exists());
					TEST_TRUE(Element->ScrollToEndUntil(Document));
					TEST_TRUE(Document->IsVisible());
				});
			});

			Describe("should fail if scrollable element reaches the end and the desired element hasn't been found", [this]()
			{
				BeforeEach([this]() {
					SuiteWidget->ScrollDocumentsToTop();
				});

				It("", EAsyncExecution::ThreadPool, [this]()
				{
					FDriverElementRef Element = Driver->FindElement(By::Path("#Documents//Tiles"));
					Driver->Wait(Until::ElementIsScrolledToBeginning(Element, FWaitTimeout::InSeconds(3)));
					TEST_TRUE(Element->IsScrolledToBeginning());

					FDriverElementRef Document = Driver->FindElement(By::Id("NotDefined"));
					TEST_FALSE(Element->ScrollToEndUntil(Document));
				});
			});
		});

		Describe("Type", [this]()
		{
			It("should focus the element and type the characters of the specified string", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverElementRef Element = Driver->FindElement(By::Id("A1"));
				Element->Type(TEXT("ABCDEFGHIJKLMNOPQRSTUVWXYZ"));
				TEST_EQUAL(SuiteViewModel->GetFormString(EFormElement::A1), TEXT("ABCDEFGHIJKLMNOPQRSTUVWXYZ"));
			});

			It("should focus the element and type the specified FKey", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverElementRef Element = Driver->FindElement(By::Id("A2"));
				Element->Type(EKeys::Z);
				TEST_EQUAL(SuiteViewModel->GetFormString(EFormElement::A2), TEXT("Z"));
			});

			It("should focus the element and type the specified array of FKeys", EAsyncExecution::ThreadPool, [this]()
			{
				TArray<FKey> Keys;
				Keys.Add(EKeys::Z);
				Keys.Add(EKeys::Y);
				Keys.Add(EKeys::X);
				Keys.Add(EKeys::W);
				Keys.Add(EKeys::V);
				Keys.Add(EKeys::U);
				Keys.Add(EKeys::T);
				Keys.Add(EKeys::S);
				Keys.Add(EKeys::R);
				Keys.Add(EKeys::Q);
				Keys.Add(EKeys::P);
				Keys.Add(EKeys::O);
				Keys.Add(EKeys::N);
				Keys.Add(EKeys::M);
				Keys.Add(EKeys::L);
				Keys.Add(EKeys::K);
				Keys.Add(EKeys::J);
				Keys.Add(EKeys::I);
				Keys.Add(EKeys::H);
				Keys.Add(EKeys::G);
				Keys.Add(EKeys::F);
				Keys.Add(EKeys::E);
				Keys.Add(EKeys::D);
				Keys.Add(EKeys::C);
				Keys.Add(EKeys::B);
				Keys.Add(EKeys::A);

				Driver->FindElement(By::Id("B1"))->Type(Keys);
				TEST_EQUAL(SuiteViewModel->GetFormString(EFormElement::B1), TEXT("ZYXWVUTSRQPONMLKJIHGFEDCBA"));
			});

			It("should focus the element and type the characters of the specified string including tabs", EAsyncExecution::ThreadPool, [this]()
			{
				Driver->FindElement(By::Id("A1"))->Type(TEXT("ABCD\tEFGH\tIJKL\tMNOP\tQRST\tUVWXYZ"));
				TEST_EQUAL(SuiteViewModel->GetFormString(EFormElement::A1), TEXT("ABCD"));
				TEST_EQUAL(SuiteViewModel->GetFormString(EFormElement::A2), TEXT("EFGH"));
				TEST_EQUAL(SuiteViewModel->GetFormString(EFormElement::B1), TEXT("IJKL"));
				TEST_EQUAL(SuiteViewModel->GetFormString(EFormElement::B2), TEXT("MNOP"));
				TEST_EQUAL(SuiteViewModel->GetFormString(EFormElement::C1), TEXT("QRST"));
				TEST_EQUAL(SuiteViewModel->GetFormString(EFormElement::C2), TEXT("UVWXYZ"));
			});
		});

		Describe("Press", [this]()
		{
			It("should focus the element and simulate a key down event of the specified key", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverElementRef Element = Driver->FindElement(By::Id("B1"));
				Element->Press(EKeys::LeftShift);
				Element->Press(EKeys::LeftControl);
				Element->Press(EKeys::LeftAlt);
				Element->Press(EKeys::RightShift);
				Element->Press(EKeys::RightControl);
				Element->Press(EKeys::RightAlt);

				FModifierKeysState ModifierKeys = Driver->GetModifierKeys();
				TEST_TRUE(ModifierKeys.IsLeftShiftDown());
				TEST_TRUE(ModifierKeys.IsLeftControlDown());
				TEST_TRUE(ModifierKeys.IsLeftAltDown());
				TEST_TRUE(ModifierKeys.IsRightShiftDown());
				TEST_TRUE(ModifierKeys.IsRightControlDown());
				TEST_TRUE(ModifierKeys.IsRightAltDown());
				TEST_TRUE(Element->HasFocusedDescendants());
			});

			It("should focus the element and cause pressed modifier keys to affect subsequent keys", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverElementRef D1 = Driver->FindElement(By::Id("D1"));
				D1->Type(TEXT("12aBc34"));
				D1->Type(EKeys::Home);
				D1->Type(EKeys::Right);
				D1->Type(EKeys::Right);
				D1->Press(EKeys::LeftShift);
				D1->Type(EKeys::Right);
				D1->Type(EKeys::Right);
				D1->Type(EKeys::Right);
				D1->Release(EKeys::LeftShift);
				D1->TypeChord(EKeys::LeftControl, EKeys::X);
				TEST_TRUE(D1->HasFocusedDescendants());
				D1->TypeChord(EKeys::LeftShift, EKeys::Tab);

				FDriverElementRef C2 = Driver->FindElement(By::Id("C2"));
				C2->TypeChord(EKeys::LeftControl, EKeys::V);
				TEST_TRUE(C2->HasFocusedDescendants());

				TEST_EQUAL(SuiteViewModel->GetFormString(EFormElement::D1), TEXT("1234"));
				TEST_EQUAL(SuiteViewModel->GetFormString(EFormElement::C2), TEXT("aBc"));
				TEST_TRUE(C2->HasFocusedDescendants());
			});

			It("should focus the element and also simulate a new character event when provided a key that maps to a char", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverElementRef Element = Driver->FindElement(By::Id("C2"));
				Element->Press(EKeys::A);
				Element->Release(EKeys::A);

				Element->Press(EKeys::One);
				Element->Release(EKeys::One);

				Element->Press(TEXT('\x00E6'));
				Element->Release(TEXT('\x00E6'));

				TEST_EQUAL(SuiteViewModel->GetFormString(EFormElement::C2), TEXT("A1\x00E6"));
			});

			It("should focus the element and also simulate a mouse down at the cursor position", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverElementRef Element = Driver->FindElement(By::Id("KeyD"));
				Element->Press(EMouseButtons::Left);
				Element->Release(EMouseButtons::Left);

				TEST_EQUAL(SuiteViewModel->GetKeySequence(), TEXT("D"));
			});
		});

		Describe("Release", [this]()
		{
			It("should focus the element and simulate a key up event of the specified key", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverElementRef Element = Driver->FindElement(By::Id("C2"));
				Element->Press(EKeys::LeftShift);
				Element->Press(EKeys::LeftControl);
				Element->Press(EKeys::LeftAlt);
				Element->Press(EKeys::RightShift);
				Element->Press(EKeys::RightControl);
				Element->Press(EKeys::RightAlt);

				Element->Release(EKeys::LeftShift);
				Element->Release(EKeys::LeftControl);
				Element->Release(EKeys::LeftAlt);
				Element->Release(EKeys::RightShift);
				Element->Release(EKeys::RightControl);
				Element->Release(EKeys::RightAlt);

				FModifierKeysState ModifierKeys = Driver->GetModifierKeys();
				TEST_FALSE(ModifierKeys.IsLeftShiftDown());
				TEST_FALSE(ModifierKeys.IsLeftControlDown());
				TEST_FALSE(ModifierKeys.IsLeftAltDown());
				TEST_FALSE(ModifierKeys.IsRightShiftDown());
				TEST_FALSE(ModifierKeys.IsRightControlDown());
				TEST_FALSE(ModifierKeys.IsRightAltDown());

				Element->Press(EKeys::A);
				Element->Release(EKeys::A);

				Element->Press(EKeys::One);
				Element->Release(EKeys::One);

				Element->Press(TEXT('\x00E6'));
				Element->Release(TEXT('\x00E6'));

				TEST_EQUAL(SuiteViewModel->GetFormString(EFormElement::C2), TEXT("A1\x00E6"));
			});

			It("should focus the element and also simulate a mouse up at the cursor position", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverElementRef Element = Driver->FindElement(By::Id("KeyD"));
				Element->Press(EMouseButtons::Left);
				Element->Release(EMouseButtons::Left);

				TEST_EQUAL(SuiteViewModel->GetKeySequence(), TEXT("D"));
			});
		});

		Describe("Focus", [this]()
		{
			It("should change the default user focus to the element", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverElementRef Element = Driver->FindElement(By::Id("KeyF"));
				Element->Focus();

				TEST_TRUE(Element->IsFocused());
			});
		});

		Describe("CanFocus", [this]()
		{
			It("should return true if the element is a SWidget that can be focused", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_TRUE(Driver->FindElement(By::Id("C2"))->CanFocus());
			});

			It("should return false if the element is a SWidget that can not be focused", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_FALSE(Driver->FindElement(By::Id("Suite"))->CanFocus());
			});

			It("should return false if the element can not be found", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_FALSE(Driver->FindElement(By::Id("NotDefined"))->CanFocus());
			});
		});

		Describe("IsFocused", [this]()
		{
			It("should return true if the element is a SWidget that is currently the users focus", EAsyncExecution::ThreadPool, [this]()
			{
				Driver->FindElement(By::Id("KeyC"))->Focus();
				TEST_TRUE(Driver->FindElement(By::Id("KeyC"))->IsFocused());
			});

			It("should return false if the element is a SWidget that is not currently the users focus", EAsyncExecution::ThreadPool, [this]()
			{
				Driver->FindElement(By::Id("C2"))->Focus();
				TEST_FALSE(Driver->FindElement(By::Id("Suite"))->IsFocused());
			});

			It("should return false if the element can not be found", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_FALSE(Driver->FindElement(By::Id("NotDefined"))->IsFocused());
			});
		});

		Describe("Exists", [this]()
		{
			It("should return true if the element can locate a matching SWidget", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_TRUE(Driver->FindElement(By::Id("Piano"))->Exists());
			});

			It("should return false if the element can not locate a matching SWidget", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_FALSE(Driver->FindElement(By::Id("NotDefined"))->Exists());
			});
		});

		Describe("IsVisible", [this]()
		{
			It("should return true if the element is currently visible in the SWidget DOM being displayed", EAsyncExecution::ThreadPool, [this]()
			{
				Driver->Wait(Until::ElementIsVisible(By::Id("Piano"), FWaitTimeout::InSeconds(1)));
				TEST_TRUE(Driver->FindElement(By::Id("Piano"))->IsVisible());
			});

			It("should return false if the element is not currently visible in the SWidget DOM being displayed", EAsyncExecution::ThreadPool, [this]()
			{
				SuiteViewModel->SetPianoVisibility(EVisibility::Hidden);
				Driver->Wait(Until::ElementIsHidden(By::Id("Piano"), FWaitTimeout::InSeconds(1)));
				TEST_FALSE(Driver->FindElement(By::Id("Piano"))->IsVisible());
			});

			It("should return false if the element can not be found", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_FALSE(Driver->FindElement(By::Id("NotDefined"))->IsVisible());
			});
		});

		Describe("IsInteractable", [this]()
		{
			It("should return true if the element is a currently enabled SWidget", EAsyncExecution::ThreadPool, [this]()
			{
				Driver->Wait(Until::ElementIsVisible(By::Id("Piano"), FWaitTimeout::InSeconds(1)));
				TEST_TRUE(Driver->FindElement(By::Id("KeyB"))->IsInteractable());
			});

			It("should return false if the element is a currently not enabled SWidget", EAsyncExecution::ThreadPool, [this]()
			{
				SuiteViewModel->SetKeyResetDelay(FTimespan::FromSeconds(5));
				Driver->FindElement(By::Id("KeyB"))->Click();
				TEST_FALSE(Driver->FindElement(By::Id("KeyB"))->IsInteractable());
			});

			It("should return false if the element can not be found", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_FALSE(Driver->FindElement(By::Id("NotDefined"))->IsInteractable());
			});
		});

		Describe("GetAbsolutePosition", [this]()
		{
			LatentIt("should return the absolute screen space position of the SWidget element", EAsyncExecution::ThreadPool, [this](const FDoneDelegate& Done)
			{
				FVector2D ElementPosition = Driver->FindElement(By::Id("KeyF"))->GetAbsolutePosition();

				TSharedRef<int32, ESPMode::ThreadSafe> OriginalExistenceChecker = MakeShareable(new int32);
				TWeakPtr<int32, ESPMode::ThreadSafe> WeakExistenceChecker = OriginalExistenceChecker;

				AsyncTask(
					ENamedThreads::GameThread,
					[this, WeakExistenceChecker, ElementPosition, Done]()
					{
						const TSharedPtr<int32, ESPMode::ThreadSafe> ExistenceChecker = WeakExistenceChecker.Pin();
						if (ExistenceChecker.IsValid())
						{
							FWidgetPath WidgetPath;
							FSlateApplication::Get().FindPathToWidget(SuiteWidget->GetKeyWidget(EPianoKey::F).ToSharedRef(), WidgetPath);

							TEST_EQUAL(ElementPosition, FVector2D(WidgetPath.Widgets.Last().Geometry.LocalToAbsolute(FVector2f::ZeroVector)));
						}
						Done.Execute();
					}
				);
			});

			It("should return a ZeroVector if the element can not be found", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_EQUAL(Driver->FindElement(By::Id("NotDefined"))->GetAbsolutePosition(), FVector2D::ZeroVector);
			});
		});

		Describe("GetSize", [this]()
		{
			LatentIt("should return the screen space size of the SWidget element", EAsyncExecution::ThreadPool, [this](const FDoneDelegate& Done)
			{
				FVector2D ElementSize = Driver->FindElement(By::Id("KeyF"))->GetSize();

				TSharedRef<int32, ESPMode::ThreadSafe> OriginalExistenceChecker = MakeShareable(new int32);
				TWeakPtr<int32, ESPMode::ThreadSafe> WeakExistenceChecker = OriginalExistenceChecker;

				AsyncTask(
					ENamedThreads::GameThread,
					[this, WeakExistenceChecker, ElementSize, Done]()
					{
						const TSharedPtr<int32, ESPMode::ThreadSafe> ExistenceChecker = WeakExistenceChecker.Pin();
						if (ExistenceChecker.IsValid())
						{
							FWidgetPath WidgetPath;
							FSlateApplication::Get().FindPathToWidget(SuiteWidget->GetKeyWidget(EPianoKey::F).ToSharedRef(), WidgetPath);

							TEST_EQUAL(ElementSize, FVector2D(WidgetPath.Widgets.Last().Geometry.GetLocalSize()));
						}
						Done.Execute();
					}
				);
			});

			It("should return a ZeroVector if the element can not be found", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_EQUAL(Driver->FindElement(By::Id("NotDefined"))->GetSize(), FVector2D::ZeroVector);
			});
		});

		Describe("GetText", [this]()
		{
			It("should return the text displayed by a specified STextBlock element", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverSequenceRef Sequence = Driver->CreateSequence();
				Sequence->Actions()
					.Click(By::Id("KeyG"))
					.Click(By::Id("KeyF"))
					.Click(By::Id("KeyE"));

				TEST_TRUE(Sequence->Perform());
				TEST_EQUAL(Driver->FindElement(By::Id("KeySequence"))->GetText().ToString(), TEXT("GFE"))
			});

			It("should return the text displayed by a specified SMultiLineEditableTextBox element", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverSequenceRef Sequence = Driver->CreateSequence();
				Sequence->Actions()
					.Focus(By::Id("D1"))
					.Type(TEXT("abc\r\ndef\r\nghi"));

				TEST_TRUE(Sequence->Perform());
				TEST_EQUAL(Driver->FindElement(By::Id("D1"))->GetText().ToString(), *(FString(TEXT("abc")) + LINE_TERMINATOR + TEXT("def") + LINE_TERMINATOR + TEXT("ghi")))
			});

			It("should return a empty text if the element can not be found", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_EQUAL(Driver->FindElement(By::Id("NotDefined"))->GetText().ToString(), FString());
			});

			It("should return all text values if the element displays multiple pieces of text", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_EQUAL(Driver->FindElement(By::Id("Piano"))->GetText().ToString(), *(FString(TEXT("Ab/G#")) + LINE_TERMINATOR + TEXT("Gb/F#") + LINE_TERMINATOR + TEXT("Fb/E#") + LINE_TERMINATOR + TEXT("Eb/D#") + LINE_TERMINATOR + TEXT("Db/C#") + LINE_TERMINATOR + TEXT("Cb/B#") + LINE_TERMINATOR + TEXT("Bb/A#") + LINE_TERMINATOR + TEXT("G") + LINE_TERMINATOR + TEXT("F") + LINE_TERMINATOR + TEXT("E") + LINE_TERMINATOR + TEXT("D") + LINE_TERMINATOR + TEXT("C") + LINE_TERMINATOR + TEXT("B") + LINE_TERMINATOR + TEXT("A")));
			});

			It("should return the text value of the single child text SWidget of the specified element", EAsyncExecution::ThreadPool, [this]()
			{
				TEST_EQUAL(Driver->FindElement(By::Id("KeyA"))->GetText().ToString(), TEXT("A"));
			});
		});
	});

	Describe("Sequence", [this]()
	{
		It("should be able to perform actions across multiple windows", EAsyncExecution::ThreadPool, [this]()
		{
			FDriverSequenceRef Sequence = Driver->CreateSequence();

			Sequence->Actions()
				.Click(By::Id("KeyModifierA#"))
				.Click(By::Id("KeyA#"))
				.Click(By::Id("KeyModifierEb"))
				.Click(By::Id("KeyEb"))
				.Click(By::Id("KeyModifierB#"))
				.Click(By::Id("KeyB#"));
			
			TEST_TRUE(Sequence->Perform());
			TEST_EQUAL(SuiteViewModel->GetKeySequence(), TEXT("A#EbB#"));
		});

		It("should be performable multiple times", EAsyncExecution::ThreadPool, [this]()
		{
			FDriverElementRef KeyA = Driver->FindElement(By::Id("KeyA"));
			FDriverElementRef KeyB = Driver->FindElement(By::Id("KeyB"));
			FDriverElementRef KeyC = Driver->FindElement(By::Id("KeyC"));

			FDriverElementRef ElementA = Driver->FindElement(By::Id("A1"));
			FDriverElementRef ElementB = Driver->FindElement(By::Id("B1"));
			FDriverElementRef ElementC = Driver->FindElement(By::Id("C1"));

			FDriverSequenceRef Sequence = Driver->CreateSequence();
			Sequence->Actions()
				.Click(KeyA)
				.Click(KeyB)
				.Click(KeyC)
				.Type(ElementA, TEXT("A"))
				.Type(ElementB, TEXT("B"))
				.Type(ElementC, TEXT("C"));

			TEST_TRUE(Sequence->Perform());
			TEST_TRUE(Sequence->Perform());

			TEST_EQUAL(SuiteViewModel->GetKeySequence(), TEXT("ABCABC"));
			TEST_EQUAL(SuiteViewModel->GetFormString(EFormElement::A1), TEXT("AA"));
			TEST_EQUAL(SuiteViewModel->GetFormString(EFormElement::B1), TEXT("BB"));
			TEST_EQUAL(SuiteViewModel->GetFormString(EFormElement::C1), TEXT("CC"));
		});

		It("should be able to perform multiple types of actions in sequence", EAsyncExecution::ThreadPool, [this]()
		{
			FDriverElementRef TextBox = Driver->FindElement(By::Id("C2"));
			FDriverElementRef KeyA = Driver->FindElement(By::Id("KeyA"));

			FDriverSequenceRef Sequence = Driver->CreateSequence();
			Sequence->Actions()
				.Focus(TextBox)
				.Type(TEXT("1234567890"))
				.Click(KeyA)
				.Focus(TextBox)
				.Type(EKeys::Home)
				.Press(EKeys::LeftShift)
				.Type(EKeys::Right)
				.Type(EKeys::Right)
				.Type(EKeys::Right)
				.Release(EKeys::LeftShift)
				.Type(EKeys::Delete)
				.Type(EKeys::End)
				.Type(EKeys::Left)
				.Type(EKeys::Left)
				.Type(TEXT("ABC"));

			TEST_TRUE(Sequence->Perform());
			TEST_EQUAL(SuiteViewModel->GetKeySequence(), TEXT("A"));
			TEST_EQUAL(SuiteViewModel->GetFormString(EFormElement::C2), TEXT("45678ABC90"));
			TEST_TRUE(TextBox->HasFocusedDescendants());
		});

		Describe("Wait", [this]()
		{
			Describe("Until::ElementExists", [this]()
			{
				It("should pause sequence execution until the specified element exists", EAsyncExecution::ThreadPool, [this]()
				{
					FDriverElementRef ElementB = Driver->FindElement(By::Id("KeyB"));
					ElementB->Focus();

					TSharedRef<int32, ESPMode::ThreadSafe> OriginalExistenceChecker = MakeShareable(new int32);
					TWeakPtr<int32, ESPMode::ThreadSafe> WeakExistenceChecker = OriginalExistenceChecker;
					AsyncTask(
						ENamedThreads::GameThread,
						[this, WeakExistenceChecker]()
						{
							const TSharedPtr<int32, ESPMode::ThreadSafe> ExistenceChecker = WeakExistenceChecker.Pin();
							if (ExistenceChecker.IsValid())
							{
								SuiteWidget->RemoveContents();
								FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this, WeakExistenceChecker](float Delta){
									const TSharedPtr<int32, ESPMode::ThreadSafe> TickerExistenceChecker = WeakExistenceChecker.Pin();
									if (TickerExistenceChecker.IsValid())
									{
										SuiteWidget->RestoreContents();
									}
									return false;
								}), 2);
							}
						}
					);

					Driver->GetConfiguration()->ImplicitWait = FTimespan::FromSeconds(0.5);
					FDriverElementRef ElementA = Driver->FindElement(By::Id("KeyA"));

					FDriverSequenceRef Sequence = Driver->CreateSequence();
					Sequence->Actions()
						.Wait(Until::ElementExists(ElementA, FWaitTimeout::InSeconds(3)))
						.Focus(ElementA);

					TEST_TRUE(Sequence->Perform());
					TEST_TRUE(ElementA->IsFocused());
				});

				Describe("should cancel further sequence execution after timing out", [this]()
				{
					It("", EAsyncExecution::ThreadPool, [this]()
					{
						FDriverElementRef ElementB = Driver->FindElement(By::Id("KeyB"));
						ElementB->Focus();

						Driver->GetConfiguration()->ImplicitWait = FTimespan::FromSeconds(0.5);
						FDriverElementRef ElementA = Driver->FindElement(By::Id("KeyA"));

						Async(EAsyncExecution::TaskGraphMainThread, [this]()
						{
							// Note that we have to call this method in GameThread to avoid assertions
							SuiteWidget->RemoveContents();
						}).Get();

						FDriverSequenceRef Sequence = Driver->CreateSequence();
						Sequence->Actions()
							.Wait(Until::ElementExists(ElementA, FWaitInterval::InSeconds(0.25), FWaitTimeout::InSeconds(1)))
							.Focus(ElementA);

						TEST_FALSE(Sequence->Perform());
						TEST_FALSE(ElementA->IsFocused());
					});

					AfterEach([this]()
					{
						SuiteWidget->RestoreContents();
					});
				});
			});

			Describe("Until::ElementIsVisible", [this]()
			{
				It("should pause sequence execution until the specified element becomes visible", EAsyncExecution::ThreadPool, [this]()
				{
					FDriverElementRef ElementB = Driver->FindElement(By::Id("KeyB"));
					ElementB->Focus();

					TSharedRef<int32, ESPMode::ThreadSafe> OriginalExistenceChecker = MakeShareable(new int32);
					TWeakPtr<int32, ESPMode::ThreadSafe> WeakExistenceChecker = OriginalExistenceChecker;
					SuiteViewModel->SetPianoVisibility(EVisibility::Collapsed);
					AsyncTask(
						ENamedThreads::GameThread,
						[this, WeakExistenceChecker]()
						{
							const TSharedPtr<int32, ESPMode::ThreadSafe> ExistenceChecker = WeakExistenceChecker.Pin();
							if (ExistenceChecker.IsValid())
							{
								FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this, WeakExistenceChecker](float Delta){
									const TSharedPtr<int32, ESPMode::ThreadSafe> TickerExistenceChecker = WeakExistenceChecker.Pin();
									if (TickerExistenceChecker.IsValid())
									{
										SuiteViewModel->SetPianoVisibility(EVisibility::Visible);
									}
									return false;
								}), 2);
							}
						}
					);

					Driver->GetConfiguration()->ImplicitWait = FTimespan::FromSeconds(0.5);
					FDriverElementRef ElementA = Driver->FindElement(By::Id("KeyA"));

					FDriverSequenceRef Sequence = Driver->CreateSequence();
					Sequence->Actions()
						.Wait(Until::ElementIsVisible(ElementA, FWaitTimeout::InSeconds(3)))
						.Focus(ElementA);

					TEST_TRUE(Sequence->Perform());
					TEST_TRUE(ElementA->IsFocused());
				});

				Describe("should cancel further sequence execution after timing out", [this]()
				{
					It("", EAsyncExecution::ThreadPool, [this]()
					{
						FDriverElementRef ElementB = Driver->FindElement(By::Id("KeyB"));
						ElementB->Focus();

						Driver->GetConfiguration()->ImplicitWait = FTimespan::FromSeconds(0.5);
						FDriverElementRef ElementA = Driver->FindElement(By::Id("KeyA"));

						SuiteViewModel->SetPianoVisibility(EVisibility::Collapsed);

						FDriverSequenceRef Sequence = Driver->CreateSequence();
						Sequence->Actions()
							.Wait(Until::ElementIsVisible(ElementA, FWaitInterval::InSeconds(0.25), FWaitTimeout::InSeconds(1)))
							.Focus(ElementA);

						TEST_FALSE(Sequence->Perform());
						TEST_FALSE(ElementA->IsFocused());
					});
				
					AfterEach([this]() {
						SuiteViewModel->SetPianoVisibility(EVisibility::Visible);
					});
				});
			});

			Describe("Until::ElementIsInteractable", [this]()
			{
				It("should pause sequence execution until the specified element becomes visible", EAsyncExecution::ThreadPool, [this]()
				{
					TSharedRef<int32, ESPMode::ThreadSafe> OriginalExistenceChecker = MakeShareable(new int32);
					TWeakPtr<int32, ESPMode::ThreadSafe> WeakExistenceChecker = OriginalExistenceChecker;
					SuiteViewModel->SetKeyResetDelay(FTimespan::FromSeconds(2));
					AsyncTask(
						ENamedThreads::GameThread,
						[this, WeakExistenceChecker]()
						{
							const TSharedPtr<int32, ESPMode::ThreadSafe> ExistenceChecker = WeakExistenceChecker.Pin();
							if (ExistenceChecker.IsValid())
							{
								FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this, WeakExistenceChecker](float Delta){
									const TSharedPtr<int32, ESPMode::ThreadSafe> TickerExistenceChecker = WeakExistenceChecker.Pin();
									if (TickerExistenceChecker.IsValid())
									{
										SuiteViewModel->SetKeyResetDelay(FTimespan::Zero());
									}
									return false;
								}), 2);
							}
						}
					);

					Driver->GetConfiguration()->ImplicitWait = FTimespan::FromSeconds(0.5);
					FDriverElementRef ElementB = Driver->FindElement(By::Id("KeyB"));
					FDriverElementRef ElementA = Driver->FindElement(By::Id("KeyA"));
					ElementA->Click();

					FDriverSequenceRef Sequence = Driver->CreateSequence();
					Sequence->Actions()
						.Wait(Until::ElementIsInteractable(ElementA, FWaitTimeout::InSeconds(3)))
						.Focus(ElementB);

					TEST_TRUE(Sequence->Perform());
					TEST_TRUE(ElementB->IsFocused());
				});

				Describe("should cancel further sequence execution after timing out", [this]()
				{
					BeforeEach([this]() {
						SuiteViewModel->SetKeyResetDelay(FTimespan::FromSeconds(5.0));
					});

					It("", EAsyncExecution::ThreadPool, [this]()
					{
						Driver->GetConfiguration()->ImplicitWait = FTimespan::FromSeconds(0.5);
						FDriverElementRef ElementB = Driver->FindElement(By::Id("KeyB"));
						FDriverElementRef ElementA = Driver->FindElement(By::Id("KeyA"));

						ElementA->Click();

						FDriverSequenceRef Sequence = Driver->CreateSequence();
						Sequence->Actions()
							.Wait(Until::ElementIsInteractable(ElementA, FWaitInterval::InSeconds(0.25), FWaitTimeout::InSeconds(1)))
							.Focus(ElementB);

						TEST_FALSE(Sequence->Perform());
						TEST_FALSE(ElementB->IsFocused());
					});

					AfterEach([this]() {
						SuiteViewModel->SetKeyResetDelay(FTimespan::FromSeconds(0.0));
					});
				});
			});
		});

		Describe("MoveToElement", [this]()
		{
			It("should simulate a cursor move over the element", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverElementRef Element = Driver->FindElement(By::Id("KeyG"));

				FDriverSequenceRef Sequence = Driver->CreateSequence();
				Sequence->Actions()
					.MoveToElement(Element);

				TEST_TRUE(Sequence->Perform());
				TEST_TRUE(Element->IsHovered());
			});

			It("should simulate a cursor move over the elements, in sequence", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverElementRef ElementA = Driver->FindElement(By::Id("KeyA"));
				FDriverElementRef ElementB = Driver->FindElement(By::Id("KeyB"));
				FDriverElementRef ElementC = Driver->FindElement(By::Id("KeyC"));
				FDriverElementRef ElementD = Driver->FindElement(By::Id("KeyD"));
				FDriverElementRef ElementE = Driver->FindElement(By::Id("KeyE"));
				FDriverElementRef ElementF = Driver->FindElement(By::Id("KeyF"));
				FDriverElementRef ElementG = Driver->FindElement(By::Id("KeyG"));

				SuiteViewModel->SetRecordKeyHoverSequence(true);

				FDriverSequenceRef Sequence = Driver->CreateSequence();
				Sequence->Actions()
					.MoveToElement(ElementA)
					.MoveToElement(ElementB)
					.MoveToElement(ElementC)
					.MoveToElement(ElementD)
					.MoveToElement(ElementE)
					.MoveToElement(ElementF)
					.MoveToElement(ElementG)
					.MoveToElement(ElementF)
					.MoveToElement(ElementE)
					.MoveToElement(ElementD)
					.MoveToElement(ElementC)
					.MoveToElement(ElementB)
					.MoveToElement(ElementA);

				TEST_TRUE(Sequence->Perform());
				TEST_EQUAL(SuiteViewModel->GetKeySequence(), TEXT("ABCDEFGFEDCBA"));
			});

			Describe("attempt to scroll the element into view (Scrolling Down) if it exists but isn't visible and then move the cursor over the element", [this]()
			{
				BeforeEach([this]() {
					SuiteWidget->ScrollDocumentsToTop();
				});

				It("", EAsyncExecution::ThreadPool, [this]()
				{
					FDriverElementRef ScrollBox = Driver->FindElement(By::Path("#Documents//<SScrollBox>"));
					Driver->Wait(Until::ElementIsScrolledToBeginning(ScrollBox, FWaitTimeout::InSeconds(3)));

					FDriverElementRef Element = Driver->FindElement(By::Path("#Documents//<SScrollBox>//#Document150"));
					FDriverSequenceRef Sequence = Driver->CreateSequence();
					Sequence->Actions()
						.MoveToElement(Element);

					TEST_TRUE(Sequence->Perform());
					TEST_TRUE(Element->IsHovered());
				});
			});

			Describe("attempt to scroll the element into view (Scrolling Up) if it exists but isn't visible and then move the cursor over the element", [this]()
			{
				BeforeEach([this]() {
					SuiteWidget->ScrollDocumentsToBottom();
				});

				It("", EAsyncExecution::ThreadPool, [this]()
				{
					FDriverElementRef ScrollBox = Driver->FindElement(By::Path("#Documents//<SScrollBox>"));
					Driver->Wait(Until::ElementIsScrolledToEnd(ScrollBox, FWaitTimeout::InSeconds(3)));

					FDriverElementRef Element = Driver->FindElement(By::Path("#Documents//<SScrollBox>//#Document50"));
					FDriverSequenceRef Sequence = Driver->CreateSequence();
					Sequence->Actions()
						.MoveToElement(Element);

					TEST_TRUE(Sequence->Perform());
					TEST_TRUE(Element->IsHovered());
				});
			});
		});

		Describe("MoveByOffset", [this]()
		{
			It("should simulate a cursor move which is an offset of the current cursor position", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverElementRef Element = Driver->FindElement(By::Id("KeyD"));
				Element->Hover();

				const FVector2D CursorPosition = Driver->GetCursorPosition();
				const FVector2D ExpectedPosition(CursorPosition.X + 15, CursorPosition.Y + 15);

				FDriverSequenceRef Sequence = Driver->CreateSequence();
				Sequence->Actions()
					.MoveByOffset(0, 10)
					.MoveByOffset(10, 0)
					.MoveByOffset(5, 5);

				TEST_TRUE(Sequence->Perform());
				TEST_EQUAL(Driver->GetCursorPosition(), ExpectedPosition);
			});
		});

		Describe("ScrollBy", [this]()
		{
			Describe("should simulate a mouse wheel event at the current cursor position by the specified negative delta", [this]()
			{
				BeforeEach([this]() {
					SuiteWidget->ScrollDocumentsToTop();
				});

				It("", EAsyncExecution::ThreadPool, [this]()
				{
					FDriverElementRef Element = Driver->FindElement(By::Path("#Documents//List"));
					Driver->Wait(Until::ElementIsScrolledToBeginning(Element, FWaitTimeout::InSeconds(3)));
					TEST_TRUE(Element->IsScrolledToBeginning());
					Element->Hover();

					FDriverSequenceRef Sequence = Driver->CreateSequence();
					Sequence->Actions()
						.ScrollBy(-1);

					TEST_TRUE(Sequence->Perform());
					TEST_FALSE(Element->IsScrolledToBeginning());
				});
			});

			Describe("should simulate a mouse wheel event at the current cursor position by the specified positive delta", [this]()
			{
				BeforeEach([this]() {
					SuiteWidget->ScrollDocumentsToBottom();
				});

				It("", EAsyncExecution::ThreadPool, [this]()
				{
					FDriverElementRef Element = Driver->FindElement(By::Path("#Documents//Tiles"));
					Driver->Wait(Until::ElementIsScrolledToEnd(Element, FWaitTimeout::InSeconds(3)));
					TEST_TRUE(Element->IsScrolledToEnd());
					Element->Hover();

					FDriverSequenceRef Sequence = Driver->CreateSequence();
					Sequence->Actions()
						.ScrollBy(1);

					TEST_TRUE(Sequence->Perform());
					TEST_FALSE(Element->IsScrolledToEnd());
				});
			});
		});

		Describe("ScrollToBeginning", [this]()
		{
			Describe("should move the cursor over the element and simulate a positive delta mouse wheel event by the specified amount", [this]()
			{
				BeforeEach([this]() {
					SuiteWidget->ScrollDocumentsToBottom();
				});

				It("", EAsyncExecution::ThreadPool, [this]()
				{
					FDriverElementRef Element = Driver->FindElement(By::Path("#Documents//Tiles"));
					Driver->Wait(Until::ElementIsScrolledToEnd(Element, FWaitTimeout::InSeconds(3)));
					TEST_TRUE(Element->IsScrolledToEnd());

					FDriverSequenceRef Sequence = Driver->CreateSequence();
					Sequence->Actions()
						.ScrollToBeginning(Element, 1);

					TEST_TRUE(Sequence->Perform());
					TEST_TRUE(Element->IsScrolledToBeginning());
				});
			});

			Describe("should move the cursor over the element and simulate positive delta mouse wheel events until the beginning is reached", [this]()
			{
				BeforeEach([this]() {
					SuiteWidget->ScrollDocumentsToBottom();
				});

				It("", EAsyncExecution::ThreadPool, [this]()
				{
					FDriverElementRef Element = Driver->FindElement(By::Path("#Documents//List"));
					Driver->Wait(Until::ElementIsScrolledToEnd(Element, FWaitTimeout::InSeconds(3)));
					TEST_TRUE(Element->IsScrolledToEnd());

					FDriverSequenceRef Sequence = Driver->CreateSequence();
					Sequence->Actions()
						.ScrollToBeginning(Element);

					TEST_TRUE(Sequence->Perform());
					TEST_TRUE(Element->IsScrolledToBeginning());
				});
			});
		});

		Describe("ScrollToBeginningUntil", [this]()
		{
			Describe("should move the cursor over the element and simulate positive mouse wheel events until the specified sub-element is visible or the beginning is reached", [this]()
			{
				BeforeEach([this]() {
					SuiteWidget->ScrollDocumentsToBottom();
				});

				It("", EAsyncExecution::ThreadPool, [this]()
				{
					FDriverElementRef Element = Driver->FindElement(By::Path("#Documents//List"));
					Driver->Wait(Until::ElementIsScrolledToEnd(Element, FWaitTimeout::InSeconds(3)));
					TEST_TRUE(Element->IsScrolledToEnd());

					FDriverElementRef Document = Driver->FindElement(By::Path("#Documents//List//#Document150"));
					TEST_FALSE(Document->Exists());

					FDriverSequenceRef Sequence = Driver->CreateSequence();
					Sequence->Actions()
						.ScrollToBeginningUntil(Element, Document);

					TEST_TRUE(Sequence->Perform());
					TEST_TRUE(Document->IsVisible());
				});
			});

			Describe("should fail if scrollable element reaches the beginning and the desired element hasn't been found", [this]()
			{
				BeforeEach([this]() {
					SuiteWidget->ScrollDocumentsToBottom();
				});

				It("", EAsyncExecution::ThreadPool, [this]()
				{
					FDriverElementRef Element = Driver->FindElement(By::Path("#Documents//Tiles"));
					Driver->Wait(Until::ElementIsScrolledToEnd(Element, FWaitTimeout::InSeconds(3)));
					TEST_TRUE(Element->IsScrolledToEnd());

					FDriverElementRef Document = Driver->FindElement(By::Id("NotDefined"));
					FDriverSequenceRef Sequence = Driver->CreateSequence();
					Sequence->Actions()
						.ScrollToBeginningUntil(Element, Document);

					TEST_FALSE(Sequence->Perform());
				});
			});
		});

		Describe("ScrollToEnd", [this]()
		{
			Describe("should move the cursor over the element and simulate a negative delta mouse wheel event by the specified amount", [this]()
			{
				BeforeEach([this]() {
					SuiteWidget->ScrollDocumentsToTop();
				});

				It("", EAsyncExecution::ThreadPool, [this]()
				{
					FDriverElementRef Element = Driver->FindElement(By::Path("#Documents//List"));
					Driver->Wait(Until::ElementIsScrolledToBeginning(Element, FWaitTimeout::InSeconds(3)));
					TEST_TRUE(Element->IsScrolledToBeginning());

					FDriverSequenceRef Sequence = Driver->CreateSequence();
					Sequence->Actions()
						.ScrollToEnd(Element, 1);

					TEST_TRUE(Sequence->Perform());
					TEST_TRUE(Element->IsScrolledToEnd());
				});
			});
			
			Describe("should move the cursor over the element and simulate negative delta mouse wheel events until the end is reached", [this]()
			{
				BeforeEach([this]() {
					SuiteWidget->ScrollDocumentsToTop();
				});

				It("", EAsyncExecution::ThreadPool, [this]()
				{
					FDriverElementRef Element = Driver->FindElement(By::Path("#Documents//Tiles"));
					Driver->Wait(Until::ElementIsScrolledToBeginning(Element, FWaitTimeout::InSeconds(3)));
					TEST_TRUE(Element->IsScrolledToBeginning());

					FDriverSequenceRef Sequence = Driver->CreateSequence();
					Sequence->Actions()
						.ScrollToEnd(Element);

					TEST_TRUE(Sequence->Perform());
					TEST_TRUE(Element->IsScrolledToEnd());
				});
			});
		});

		Describe("ScrollToEndUntil", [this]()
		{
			Describe("should move the cursor over the element and simulate a mouse wheel events until the specified sub-element is visible", [this]()
			{
				BeforeEach([this]() {
					SuiteWidget->ScrollDocumentsToTop();
				});

				It("", EAsyncExecution::ThreadPool, [this]()
				{
					FDriverElementRef Element = Driver->FindElement(By::Path("#Documents//Tiles"));
					Driver->Wait(Until::ElementIsScrolledToBeginning(Element, FWaitTimeout::InSeconds(3)));
					TEST_TRUE(Element->IsScrolledToBeginning());

					FDriverElementRef Document = Driver->FindElement(By::Path("#Documents//Tiles//#Document50"));
					TEST_FALSE(Document->Exists());

					FDriverSequenceRef Sequence = Driver->CreateSequence();
					Sequence->Actions()
						.ScrollToEndUntil(Element, Document);

					TEST_TRUE(Sequence->Perform());
					TEST_TRUE(Document->IsVisible());
				});
			});

			Describe("should fail if scrollable element reaches the end and the desired element hasn't been found", [this]()
			{
				BeforeEach([this]() {
					SuiteWidget->ScrollDocumentsToTop();
				});

				It("", EAsyncExecution::ThreadPool, [this]()
				{
					FDriverElementRef Element = Driver->FindElement(By::Path("#Documents//List"));
					Driver->Wait(Until::ElementIsScrolledToBeginning(Element, FWaitTimeout::InSeconds(3)));
					TEST_TRUE(Element->IsScrolledToBeginning());

					FDriverElementRef Document = Driver->FindElement(By::Id("NotDefined"));
					FDriverSequenceRef Sequence = Driver->CreateSequence();
					Sequence->Actions()
						.ScrollToEndUntil(Element, Document);

					TEST_FALSE(Sequence->Perform());
				});
			});
		});
		
		Describe("Click", [this]()
		{
			It("should simulate a click at the cursors current position", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverElementRef Element = Driver->FindElement(By::Id("KeyG"));
				Element->Hover();

				FDriverSequenceRef Sequence = Driver->CreateSequence();
				Sequence->Actions()
					.Click(EMouseButtons::Left);

				TEST_TRUE(Sequence->Perform());
				TEST_EQUAL(SuiteViewModel->GetKeySequence(), TEXT("G"));
			});

			It("should simulate a click at the cursors current position, in sequence", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverElementRef Element = Driver->FindElement(By::Id("KeyB"));
				Element->Hover();

				FDriverSequenceRef Sequence = Driver->CreateSequence();
				Sequence->Actions()
					.Click(EMouseButtons::Left)
					.Click(EMouseButtons::Left);

				TEST_TRUE(Sequence->Perform());
				TEST_EQUAL(SuiteViewModel->GetKeySequence(), TEXT("BB"));
			});
		});

		Describe("DoubleClick", [this]()
		{
			It("should simulate a double click at the cursors current position", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverElementRef Element = Driver->FindElement(By::Id("KeyF"));
				Element->Hover();

				FDriverSequenceRef Sequence = Driver->CreateSequence();
				Sequence->Actions()
					.DoubleClick(EMouseButtons::Left);

				TEST_TRUE(Sequence->Perform());
				TEST_EQUAL(SuiteViewModel->GetKeySequence(), TEXT("FF"));
			});

			It("should simulate a double click at the cursors current position, in sequence", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverElementRef Element = Driver->FindElement(By::Id("KeyF"));
				Element->Hover();

				FDriverSequenceRef Sequence = Driver->CreateSequence();
				Sequence->Actions()
					.DoubleClick(EMouseButtons::Left)
					.DoubleClick(EMouseButtons::Left);

				TEST_TRUE(Sequence->Perform());
				TEST_EQUAL(SuiteViewModel->GetKeySequence(), TEXT("FFFF"));
			});
		});

		Describe("Type", [this]()
		{
			It("should type the characters of the specified string into the current focus", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverElementRef Element = Driver->FindElement(By::Id("C2"));
				Element->Focus();

				FDriverSequenceRef Sequence = Driver->CreateSequence();
				Sequence->Actions()
					.Type(TEXT("abcdefghijklmnopqrstuvwxyz"));

				TEST_TRUE(Sequence->Perform());
				TEST_EQUAL(SuiteViewModel->GetFormString(EFormElement::C2), TEXT("abcdefghijklmnopqrstuvwxyz"));
			});

			It("should type the characters of the specified string even without a valid keyboard focus", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverSequenceRef Sequence = Driver->CreateSequence();
				Sequence->Actions()
					.Type(TEXT("\tABCDEFGHIJKLMNOPQRSTUVWXYZ"));

				TEST_TRUE(Sequence->Perform());
				TEST_EQUAL(SuiteViewModel->GetFormString(EFormElement::A1), TEXT("ABCDEFGHIJKLMNOPQRSTUVWXYZ"));
			});

			It("should properly handle encoded New Line characters", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverSequenceRef Sequence = Driver->CreateSequence();
				Sequence->Actions()
					.Focus(By::Id("D1"))
					.Type(TEXT("abc\ndef\r\nghi"));

				TEST_TRUE(Sequence->Perform());
				TEST_EQUAL(SuiteViewModel->GetFormString(EFormElement::D1), *(FString(TEXT("abc")) + LINE_TERMINATOR + TEXT("def") + LINE_TERMINATOR + TEXT("ghi")));
			});
		});

		Describe("Press", [this]()
		{
			It("should simulate a key down event of the specified key", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverSequenceRef Sequence = Driver->CreateSequence();
				Sequence->Actions()
					.Press(EKeys::LeftShift)
					.Press(EKeys::LeftControl)
					.Press(EKeys::LeftAlt)
					.Press(EKeys::RightShift)
					.Press(EKeys::RightControl)
					.Press(EKeys::RightAlt);

				TEST_TRUE(Sequence->Perform());

				FModifierKeysState ModifierKeys = Driver->GetModifierKeys();
				TEST_TRUE(ModifierKeys.IsLeftShiftDown());
				TEST_TRUE(ModifierKeys.IsLeftControlDown());
				TEST_TRUE(ModifierKeys.IsLeftAltDown());
				TEST_TRUE(ModifierKeys.IsRightShiftDown());
				TEST_TRUE(ModifierKeys.IsRightControlDown());
				TEST_TRUE(ModifierKeys.IsRightAltDown());
			});

			It("should cause pressed modifier keys to affect subsequent keys", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverSequenceRef Sequence = Driver->CreateSequence();
				Sequence->Actions()
					.Focus(By::Id("C2"))
					.Type(TEXT("12aBc34"))
					.Type(EKeys::Home)
					.Type(EKeys::Right)
					.Type(EKeys::Right)
					.Press(EKeys::LeftShift)
					.Type(EKeys::Right)
					.Type(EKeys::Right)
					.Type(EKeys::Right)
					.Release(EKeys::LeftShift)
					.TypeChord(EKeys::LeftControl, EKeys::X)
					.Type(EKeys::Tab)
					.TypeChord(EKeys::LeftControl, EKeys::V);

				TEST_TRUE(Sequence->Perform());
				TEST_EQUAL(SuiteViewModel->GetFormString(EFormElement::C2), TEXT("1234"));
				TEST_EQUAL(SuiteViewModel->GetFormString(EFormElement::D1), TEXT("aBc"));
			});

			It("should also simulate a new character event when provided a key that maps to a char", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverElementRef Element = Driver->FindElement(By::Id("C2"));
				Element->Focus();

				FDriverSequenceRef Sequence = Driver->CreateSequence();
				Sequence->Actions()
					.Type(EKeys::A)
					.Type(EKeys::One)
					.Type(TEXT('\x00E6'));

				TEST_TRUE(Sequence->Perform());
				TEST_EQUAL(SuiteViewModel->GetFormString(EFormElement::C2), TEXT("A1\x00E6"));
			});
		});

		Describe("Release", [this]()
		{
			It("should simulate a key up event of the specified key", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverSequenceRef PressSequence = Driver->CreateSequence();
				PressSequence->Actions()
					.Press(EKeys::LeftShift)
					.Press(EKeys::LeftControl)
					.Press(EKeys::LeftAlt)
					.Press(EKeys::RightShift)
					.Press(EKeys::RightControl)
					.Press(EKeys::RightAlt);
				TEST_TRUE(PressSequence->Perform());

				FDriverSequenceRef ReleaseSequence = Driver->CreateSequence();
				ReleaseSequence->Actions()
					.Release(EKeys::LeftShift)
					.Release(EKeys::LeftControl)
					.Release(EKeys::LeftAlt)
					.Release(EKeys::RightShift)
					.Release(EKeys::RightControl)
					.Release(EKeys::RightAlt);
				TEST_TRUE(ReleaseSequence->Perform());

				FModifierKeysState ModifierKeys = Driver->GetModifierKeys();
				TEST_FALSE(ModifierKeys.IsLeftShiftDown());
				TEST_FALSE(ModifierKeys.IsLeftControlDown());
				TEST_FALSE(ModifierKeys.IsLeftAltDown());
				TEST_FALSE(ModifierKeys.IsRightShiftDown());
				TEST_FALSE(ModifierKeys.IsRightControlDown());
				TEST_FALSE(ModifierKeys.IsRightAltDown());
			});
		});

		Describe("Focus", [this]()
		{
			It("should change the default user focus to the element", EAsyncExecution::ThreadPool, [this]()
			{
				FDriverElementRef Element = Driver->FindElement(By::Id("KeyA"));

				FDriverSequenceRef Sequence = Driver->CreateSequence();
				Sequence->Actions()
					.Focus(Element);

				TEST_TRUE(Sequence->Perform());
				TEST_TRUE(Element->IsFocused());
			});
		});
	});

	AfterEach([this]() {
		Driver.Reset();
		IAutomationDriverModule::Get().Disable();
	});
}

#undef TEST_TRUE
#undef TEST_FALSE
#undef TEST_EQUAL
#undef EPIC_TEST_BOOLEAN_
