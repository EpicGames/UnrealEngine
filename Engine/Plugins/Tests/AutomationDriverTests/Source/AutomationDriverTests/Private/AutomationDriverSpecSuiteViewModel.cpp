// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomationDriverSpecSuiteViewModel.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Misc/DateTime.h"

class FAutomationDriverSpecSuiteViewModel
	: public IAutomationDriverSpecSuiteViewModel
	, public TSharedFromThis<FAutomationDriverSpecSuiteViewModel>
{
public:

	virtual ~FAutomationDriverSpecSuiteViewModel()
	{ }

	virtual FText GetFormText(EFormElement Element) const override
	{
		return FormTextMap.FindOrAdd(Element);
	}

	virtual FString GetFormString(EFormElement Element) const override
	{
		return FormTextMap.FindOrAdd(Element).ToString();
	}

	virtual void OnFormTextCommitted(const FText& InText, ETextCommit::Type InCommitType, EFormElement Element) override
	{
		FormTextMap.Add(Element, InText);
	}

	virtual void OnFormTextChanged(const FText& InText, EFormElement Element) override
	{
		FormTextMap.Add(Element, InText);
	}

	virtual TArray<TSharedRef<FDocumentInfo>>& GetDocuments() override
	{
		return Documents;
	}

	virtual FReply DocumentButtonClicked(TSharedRef<FDocumentInfo> Document) override
	{
		return FReply::Handled();
	}

	virtual bool IsKeyEnabled(EPianoKey Key) const override
	{
		if (KeyResetDelay.IsZero())
		{
			return true;
		}

		const FDateTime* LastClick = LastKeyClick.Find(Key);

		if (LastClick == nullptr)
		{
			return true;
		}

		return FDateTime::Now() - *LastClick > KeyResetDelay;
	}

	virtual FReply KeyClicked(EPianoKey Key) override
	{
		ActionSequence += FPianoKeyExtensions::ToString(Key);

		LastKeyClick.Add(Key, FDateTime::Now());
		return FReply::Handled();
	}

	virtual void KeyHovered(EPianoKey Key) override
	{
		if (bRecordHoverSequence)
		{
			ActionSequence += FPianoKeyExtensions::ToString(Key);
		}
	}

	virtual FString GetKeySequence() const override
	{
		return ActionSequence;
	}

	virtual FText GetKeySequenceText() const override
	{
		return FText::FromString(ActionSequence);
	}

	virtual FTimespan GetKeyResetDelay() const override
	{
		return KeyResetDelay;
	}

	virtual void SetKeyResetDelay(FTimespan Delay) override
	{
		KeyResetDelay = Delay;
	}

	virtual void SetRecordKeyHoverSequence(bool Value) override
	{
		bRecordHoverSequence = Value;
	}

	virtual bool GetRecordKeyHoverSequence() const override
	{
		return bRecordHoverSequence;
	}

	virtual void SetPianoVisibility(EVisibility Value) override
	{
		PianoVisibility = Value;
	}

	virtual EVisibility GetPianoVisibility() const override
	{
		return PianoVisibility;
	}

	virtual void Reset() override
	{
		PianoVisibility = EVisibility::Visible;
		bRecordHoverSequence = false;
		ActionSequence.Empty();
		LastKeyClick.Empty();
		FormTextMap.Empty();
		KeyResetDelay = FTimespan::Zero();
	}

private:

	FAutomationDriverSpecSuiteViewModel()
		: ActionSequence()
		, KeyResetDelay()
		, LastKeyClick()
	{ }

	void Initialize()
	{
		Documents.Empty(200);
		for (int32 Index = 0; Index < 200; ++Index)
		{
			Documents.Add(MakeShareable(new FDocumentInfo(FText::FromString("Document " + FString::FormatAsNumber(Index + 1)), Index + 1)));
		}

		Reset();
	}

private:

	mutable TMap<EFormElement, FText> FormTextMap;

	FString ActionSequence;
	bool bRecordHoverSequence;
	EVisibility PianoVisibility;

	TArray<TSharedRef<FDocumentInfo>> Documents;

	FTimespan KeyResetDelay;
	TMap<EPianoKey, FDateTime> LastKeyClick;

	friend FSpecSuiteViewModelFactory;
};

TSharedRef<IAutomationDriverSpecSuiteViewModel> FSpecSuiteViewModelFactory::Create()
{
	TSharedRef<FAutomationDriverSpecSuiteViewModel> Instance = MakeShareable(new FAutomationDriverSpecSuiteViewModel());
	Instance->Initialize();
	return Instance;
}
