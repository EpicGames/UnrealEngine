// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Internationalization/Text.h"

class FReply;
namespace ETextCommit { enum Type : int; }
struct EVisibility;

enum class EPianoKey : uint8
{
	AFlat,
	A,
	ASharp,
	BFlat,
	B,
	BSharp,
	CFlat,
	C,
	CSharp,
	DFlat,
	D,
	DSharp,
	EFlat,
	E,
	ESharp,
	FFlat,
	F,
	FSharp,
	GFlat,
	G,
	GSharp,
};

class FPianoKeyExtensions
{
public:

	static FString ToString(EPianoKey Key)
	{
		switch (Key)
		{
		case EPianoKey::AFlat:
			return TEXT("Ab");
			break;
		case EPianoKey::A:
			return TEXT("A");
			break;
		case EPianoKey::ASharp:
			return TEXT("A#");
			break;
		case EPianoKey::BFlat:
			return TEXT("Bb");
			break;
		case EPianoKey::B:
			return TEXT("B");
			break;
		case EPianoKey::BSharp:
			return TEXT("B#");
			break;
		case EPianoKey::CFlat:
			return TEXT("Cb");
			break;
		case EPianoKey::C:
			return TEXT("C");
			break;
		case EPianoKey::CSharp:
			return TEXT("C#");
			break;
		case EPianoKey::DFlat:
			return TEXT("Db");
			break;
		case EPianoKey::D:
			return TEXT("D");
			break;
		case EPianoKey::DSharp:
			return TEXT("D#");
			break;
		case EPianoKey::EFlat:
			return TEXT("Eb");
			break;
		case EPianoKey::E:
			return TEXT("E");
			break;
		case EPianoKey::ESharp:
			return TEXT("E#");
			break;
		case EPianoKey::FFlat:
			return TEXT("Fb");
			break;
		case EPianoKey::F:
			return TEXT("F");
			break;
		case EPianoKey::FSharp:
			return TEXT("F#");
			break;
		case EPianoKey::GFlat:
			return TEXT("Gb");
			break;
		case EPianoKey::G:
			return TEXT("G");
			break;
		case EPianoKey::GSharp:
			return TEXT("G#");
			break;
		}

		return TEXT("Unknown Key");
	}

	static FText ToText(EPianoKey Key)
	{
		return FText::FromString(ToString(Key));
	}
};

enum class EFormElement : uint8
{
	A1,
	A2,
	B1,
	B2,
	C1,
	C2,
	D1,
};

struct FDocumentInfo
{
	FDocumentInfo(const FText& InDisplayName, int32 InNumber)
		: DisplayName(InDisplayName)
		, Number(InNumber)
	{ }

	FText DisplayName;
	int32 Number;
};

class IAutomationDriverSpecSuiteViewModel
{
public:

	virtual ~IAutomationDriverSpecSuiteViewModel()
	{ }

	/** @return The current value of the form text element */
	virtual FText GetFormText(EFormElement Element) const = 0;

	/** @return The current value of the form text element */
	virtual FString GetFormString(EFormElement Element) const = 0;

	/** Handles when form text elements commit their changes */
	virtual void OnFormTextCommitted(const FText& InText, ETextCommit::Type InCommitType, EFormElement Element) = 0;

	/** Handles when form text elements change their text */
	virtual void OnFormTextChanged(const FText& InText, EFormElement Element) = 0;

	/** @return the documents */
	virtual TArray<TSharedRef<FDocumentInfo>>& GetDocuments() = 0;

	/** @return Handles a document button being clicked */
	virtual FReply DocumentButtonClicked(TSharedRef<FDocumentInfo> Document) = 0;

	/** @return Whether the specified key button is enabled */
	virtual bool IsKeyEnabled(EPianoKey Key) const = 0;

	/** @return Handles a key being clicked */
	virtual FReply KeyClicked(EPianoKey Key) = 0;

	/** @return Handles a key being hovered over with the cursor */
	virtual void KeyHovered(EPianoKey Key) = 0;

	/** Returns the serious of recorded keys clicked */
	virtual FString GetKeySequence() const = 0;

	/** Returns the serious of recorded keys clicked */
	virtual FText GetKeySequenceText() const = 0;

	/** @return the delay between being able to click the same key again */
	virtual FTimespan GetKeyResetDelay() const = 0;

	/** Sets a new delay between being able to click the same key again */
	virtual void SetKeyResetDelay(FTimespan Delay) = 0;

	/** Sets whether the cursor hovering over a key should be recorded to the KeySequence */
	virtual void SetRecordKeyHoverSequence(bool Value) = 0;

	/** Gets whether the cursor hovering over a key should be recorded to the KeySequence */
	virtual bool GetRecordKeyHoverSequence() const = 0;

	/** Sets the visibility of the root widget composing the piano */
	virtual void SetPianoVisibility(EVisibility Value) = 0;

	/** Gets the visibility of the root widget composing the piano */
	virtual EVisibility GetPianoVisibility() const = 0;

	/** Reset the recorded sequence of keys clicked */
	virtual void Reset() = 0;
};

class FSpecSuiteViewModelFactory
{
public:
	static TSharedRef<IAutomationDriverSpecSuiteViewModel> Create();
};
