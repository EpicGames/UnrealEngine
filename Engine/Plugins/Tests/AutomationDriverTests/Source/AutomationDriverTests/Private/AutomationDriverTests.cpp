// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomationDriverTests.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FAutomationDriverTestsModule"

void FAutomationDriverTestsModule::StartupModule()
{
}

void FAutomationDriverTestsModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FAutomationDriverTestsModule, AutomationDriverTests)
