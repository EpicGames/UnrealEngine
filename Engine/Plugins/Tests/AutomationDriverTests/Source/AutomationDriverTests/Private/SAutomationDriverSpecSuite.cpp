// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAutomationDriverSpecSuite.h"
#include "AutomationDriverSpecSuiteViewModel.h"
#include "Framework/MetaData/DriverMetaData.h"

#include "Framework/Views/TableViewMetadata.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Views/STileView.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/SEditableTextBox.h"

class SAutomationDriverSpecSuiteImpl
	: public SAutomationDriverSpecSuite
{
public:
	
	virtual ~SAutomationDriverSpecSuiteImpl()
	{ }

	virtual void Construct(const FArguments& InArgs, const TSharedRef<IAutomationDriverSpecSuiteViewModel>& InViewModel) override
	{
		ViewModel = InViewModel;
		AddMetadata(FDriverMetaData::Id("Suite"));
		
		TSharedRef<SScrollBar> TilesScrollbar = SNew(SScrollBar);

		SAssignNew(WindowContents, SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(STextBlock)
			.Text(InViewModel, &IAutomationDriverSpecSuiteViewModel::GetKeySequenceText)
			.AddMetaData(FDriverMetaData::Id("KeySequence"))
			.Tag("Duplicate")
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SVerticalBox)
			.AddMetaData(FDriverMetaData::Id("UserForm"))
			.Tag("Form")
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				.AddMetaData(FDriverMetaData::Id("RowA"))
				.Tag("Rows")
				+ SHorizontalBox::Slot()
				.Padding(5, 5)
				[
					SNew(SEditableTextBox)
					.Text(InViewModel, &IAutomationDriverSpecSuiteViewModel::GetFormText, EFormElement::A1)
					.OnTextChanged(InViewModel, &IAutomationDriverSpecSuiteViewModel::OnFormTextChanged, EFormElement::A1)
					.OnTextCommitted(InViewModel, &IAutomationDriverSpecSuiteViewModel::OnFormTextCommitted, EFormElement::A1)
					.AddMetaData(FDriverMetaData::Id("A1"))
					.AddMetaData(FDriverMetaData::Id("DuplicateId"))
					.Tag("TextBox")
				]
				+ SHorizontalBox::Slot()
				.Padding(5, 5)
				[
					SNew(SEditableTextBox)
					.Text(InViewModel, &IAutomationDriverSpecSuiteViewModel::GetFormText, EFormElement::A2)
					.OnTextChanged(InViewModel, &IAutomationDriverSpecSuiteViewModel::OnFormTextChanged, EFormElement::A2)
					.OnTextCommitted(InViewModel, &IAutomationDriverSpecSuiteViewModel::OnFormTextCommitted, EFormElement::A2)
					.AddMetaData(FDriverMetaData::Id("A2"))
					.Tag("TextBox")
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				.AddMetaData(FDriverMetaData::Id("RowB"))
				.Tag("Rows")
				+ SHorizontalBox::Slot()
				.Padding(5, 5)
				[
					SNew(SEditableTextBox)
					.Text(InViewModel, &IAutomationDriverSpecSuiteViewModel::GetFormText, EFormElement::B1)
					.OnTextChanged(InViewModel, &IAutomationDriverSpecSuiteViewModel::OnFormTextChanged, EFormElement::B1)
					.OnTextCommitted(InViewModel, &IAutomationDriverSpecSuiteViewModel::OnFormTextCommitted, EFormElement::B1)
					.AddMetaData(FDriverMetaData::Id("B1"))
					.Tag("TextBox")
				]
				+ SHorizontalBox::Slot()
				.Padding(5, 5)
				[
					SNew(SEditableTextBox)
					.Text(InViewModel, &IAutomationDriverSpecSuiteViewModel::GetFormText, EFormElement::B2)
					.OnTextChanged(InViewModel, &IAutomationDriverSpecSuiteViewModel::OnFormTextChanged, EFormElement::B2)
					.OnTextCommitted(InViewModel, &IAutomationDriverSpecSuiteViewModel::OnFormTextCommitted, EFormElement::B2)
					.AddMetaData(FDriverMetaData::Id("B2"))
					.Tag("TextBox")
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				.AddMetaData(FDriverMetaData::Id("RowC"))
				.Tag("Rows")
				+ SHorizontalBox::Slot()
				.Padding(5, 5)
				[
					SNew(SEditableTextBox)
					.Text(InViewModel, &IAutomationDriverSpecSuiteViewModel::GetFormText, EFormElement::C1)
					.OnTextChanged(InViewModel, &IAutomationDriverSpecSuiteViewModel::OnFormTextChanged, EFormElement::C1)
					.OnTextCommitted(InViewModel, &IAutomationDriverSpecSuiteViewModel::OnFormTextCommitted, EFormElement::C1)
					.AddMetaData(FDriverMetaData::Id("C1"))
					.Tag("TextBox")
				]
				+ SHorizontalBox::Slot()
				.Padding(5, 5)
				[
					SNew(SEditableTextBox)
					.Text(InViewModel, &IAutomationDriverSpecSuiteViewModel::GetFormText, EFormElement::C2)
					.OnTextChanged(InViewModel, &IAutomationDriverSpecSuiteViewModel::OnFormTextChanged, EFormElement::C2)
					.OnTextCommitted(InViewModel, &IAutomationDriverSpecSuiteViewModel::OnFormTextCommitted, EFormElement::C2)
					.AddMetaData(FDriverMetaData::Id("C2"))
					.Tag("TextBox")
				]
			]
			+ SVerticalBox::Slot()
			.Padding(5, 5)
			[
				SNew(SMultiLineEditableTextBox)
				.Text(InViewModel, &IAutomationDriverSpecSuiteViewModel::GetFormText, EFormElement::D1)
				.OnTextChanged(InViewModel, &IAutomationDriverSpecSuiteViewModel::OnFormTextChanged, EFormElement::D1)
				.OnTextCommitted(InViewModel, &IAutomationDriverSpecSuiteViewModel::OnFormTextCommitted, EFormElement::D1)
				.AddMetaData(FDriverMetaData::Id("D1"))
				.Tag("TextBox")
			]
		]
		+ SVerticalBox::Slot()
		[
			SNew(SHorizontalBox)
			.AddMetaData(FDriverMetaData::Id("Documents"))
			.Tag("Documents")
			+ SHorizontalBox::Slot()
			.Padding(5, 5)
			[
				SAssignNew(DocumentList, SListView<TSharedRef<FDocumentInfo>>)
				.SelectionMode(ESelectionMode::None)
				.ListItemsSource(&InViewModel->GetDocuments())
				.OnGenerateRow(this, &SAutomationDriverSpecSuiteImpl::GenerateListRow)
				.Tag("List")
			]
			+ SHorizontalBox::Slot()
			.Padding(5, 5)
			[
				SNew(SHorizontalBox)
				.Tag("Tiles")
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SBox)
					.WidthOverride(16.0f)
					[
						TilesScrollbar
					]
				]
				+ SHorizontalBox::Slot()
				[
					SAssignNew(DocumentTiles, STileView<TSharedRef<FDocumentInfo>>)
					.ItemHeight(48)
					.ItemWidth(80)
					.SelectionMode(ESelectionMode::None)
					.ListItemsSource(&InViewModel->GetDocuments())
					.OnGenerateTile(this, &SAutomationDriverSpecSuiteImpl::GenerateListRow)
					.ExternalScrollbar(TilesScrollbar)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SBox)
					.WidthOverride(16.0f)
					[
						TilesScrollbar
					]
				]
			]
			+ SHorizontalBox::Slot()
			.Padding(5, 5)
			[
				SAssignNew(DocumentScrollBox, SScrollBox)
			]
		]
		+ SVerticalBox::Slot()
		[
			SNew(SOverlay)
			.AddMetaData(FDriverMetaData::Id("Piano"))
			.Tag("Keyboard")
			+ SOverlay::Slot()
			[
				SNew(SHorizontalBox)
				.Visibility(InViewModel, &IAutomationDriverSpecSuiteViewModel::GetPianoVisibility)
				+ SHorizontalBox::Slot()
				[
					ConstructPianoKey(EPianoKey::A)
				]
				+ SHorizontalBox::Slot()
				[
					ConstructPianoKey(EPianoKey::B)
				]
				+ SHorizontalBox::Slot()
				[
					ConstructPianoKey(EPianoKey::C)
				]
				+ SHorizontalBox::Slot()
				[
					ConstructPianoKey(EPianoKey::D)
				]
				+ SHorizontalBox::Slot()
				[
					ConstructPianoKey(EPianoKey::E)
				]
				+ SHorizontalBox::Slot()
				[
					ConstructPianoKey(EPianoKey::F)
				]
				+ SHorizontalBox::Slot()
				[
					ConstructPianoKey(EPianoKey::G)
				]
			]
			+ SOverlay::Slot()
			.VAlign(VAlign_Top)
			[
				SNew(SHorizontalBox)
				.Visibility(InViewModel, &IAutomationDriverSpecSuiteViewModel::GetPianoVisibility)
				+ SHorizontalBox::Slot()
				[
					ConstructPianoKeyModifier(EPianoKey::A, EPianoKey::B)
				]
				+ SHorizontalBox::Slot()
				[
					ConstructPianoKeyModifier(EPianoKey::B, EPianoKey::C)
				]
				+ SHorizontalBox::Slot()
				[
					ConstructPianoKeyModifier(EPianoKey::C, EPianoKey::D)
				]
				+ SHorizontalBox::Slot()
				[
					ConstructPianoKeyModifier(EPianoKey::D, EPianoKey::E)
				]
				+ SHorizontalBox::Slot()
				[
					ConstructPianoKeyModifier(EPianoKey::E, EPianoKey::F)
				]
				+ SHorizontalBox::Slot()
				[
					ConstructPianoKeyModifier(EPianoKey::F, EPianoKey::G)
				]
				+ SHorizontalBox::Slot()
				[
					ConstructPianoKeyModifier(EPianoKey::G, EPianoKey::A)
				]
			]			
		];

		for (const TSharedRef<FDocumentInfo>& Document : ViewModel->GetDocuments())
		{
			DocumentScrollBox->AddSlot()
			[
				SNew(SButton)
				.OnClicked(FOnClicked::CreateSP(ViewModel.ToSharedRef(), &IAutomationDriverSpecSuiteViewModel::DocumentButtonClicked, Document))
				.Text(Document->DisplayName)
				.Tag("Document")
				.AddMetaData(FDriverMetaData::Id(*("Document" + FString::FormatAsNumber(Document->Number))))
			];
		}

		RestoreContents();

		GetKeyWidget(EPianoKey::A)->AddMetadata<FTagMetaData>(MakeShareable(new FTagMetaData("Duplicate")));
		GetKeyWidget(EPianoKey::E)->AddMetadata(FDriverMetaData::Id("DuplicateId"));
	}

	TSharedRef<ITableRow> GenerateListRow(TSharedRef< FDocumentInfo > InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return SNew(STableRow< TSharedRef<FDocumentInfo> >, OwnerTable)
			[
				SNew(SButton)
				.OnClicked(FOnClicked::CreateSP(ViewModel.ToSharedRef(), &IAutomationDriverSpecSuiteViewModel::DocumentButtonClicked, InItem))
				.Text(InItem->DisplayName)
				.Tag("Document")
				.AddMetaData(FDriverMetaData::Id(*("Document" + FString::FormatAsNumber(InItem->Number))))
			];
	}

	TSharedRef<SWidget> CreateKeyModifierMenu(EPianoKey KeySharp, EPianoKey KeyFlat)
	{
		return SNew(SBox)
			[
				SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					[
						ConstructPianoContextMenuKey(KeyFlat)
					]
					+ SHorizontalBox::Slot()
					[
						ConstructPianoContextMenuKey(KeySharp)
					]
			];
	}

	TSharedRef<SWidget> ConstructPianoContextMenuKey(EPianoKey Key)
	{
		return ConstructPianoKeyImpl(SharedThis(this), &SAutomationDriverSpecSuiteImpl::KeyClicked, Key);
	}

	TSharedRef<SWidget> ConstructPianoKey(EPianoKey Key)
	{
		return ConstructPianoKeyImpl(ViewModel.ToSharedRef(), &IAutomationDriverSpecSuiteViewModel::KeyClicked, Key);
	}

	FReply KeyClicked(EPianoKey Key)
	{
		CloseContextMenu(Key);

		return ViewModel->KeyClicked(Key);
	}

	TSharedRef<SWidget> ConstructPianoKeyModifier(EPianoKey Key1, EPianoKey Key2)
	{
		TSharedPtr<SMenuAnchor> KeyMenuAnchor;

		const EPianoKey Key1Sharp = (EPianoKey)((int32)Key1 + 1);
		const EPianoKey Key2Flat = (EPianoKey)((int32)Key2 - 1);

		const FString Key1SharpStr = FPianoKeyExtensions::ToString((EPianoKey)((int32)Key1 + 1));
		const FString Key2FlatStr = FPianoKeyExtensions::ToString((EPianoKey)((int32)Key2 - 1));

		TSharedRef<SWidget> Widget = 
			SAssignNew(KeyMenuAnchor, SMenuAnchor)
			.Placement(MenuPlacement_Center)
			.OnGetMenuContent(this, &SAutomationDriverSpecSuiteImpl::CreateKeyModifierMenu, Key1Sharp, Key2Flat)
			[
				SNew(SButton)
				.Text(FText::FromString(Key2FlatStr + TEXT("/") + Key1SharpStr))
				.OnClicked(this, &SAutomationDriverSpecSuiteImpl::OpenContextMenu, Key1Sharp, Key2Flat)
				.HAlign(HAlign_Center)
				.AddMetaData(FDriverMetaData::Id(*(FString(TEXT("KeyModifier")) + Key1SharpStr)))
				.AddMetaData(FDriverMetaData::Id(*(FString(TEXT("KeyModifier")) + Key2FlatStr)))
				.Tag("KeyModifier")
			];
		
		PianoKeyMenus.Add(Key1Sharp, KeyMenuAnchor);
		PianoKeyMenus.Add(Key2Flat, KeyMenuAnchor);

		return Widget;
	}

	virtual FReply OpenContextMenu(EPianoKey KeySharp, EPianoKey KeyFlat)
	{
		const TSharedPtr<SMenuAnchor>* Anchor = PianoKeyMenus.Find(KeySharp);

		if (Anchor != nullptr)
		{
			(*Anchor)->SetIsOpen(true);
		}

		return FReply::Handled();
	}

	virtual FReply CloseContextMenu(EPianoKey KeySharp)
	{
		const TSharedPtr<SMenuAnchor>* Anchor = PianoKeyMenus.Find(KeySharp);

		if (Anchor != nullptr)
		{
			(*Anchor)->SetIsOpen(false);
		}

		return FReply::Handled();
	}

	virtual TSharedPtr<SWidget> GetKeyWidget(EPianoKey Key) const override
	{
		const TSharedPtr<SButton>* Button = PianoKeys.Find(Key);

		if (Button == nullptr)
		{
			return nullptr;
		}

		return *Button;
	}

	virtual void RestoreContents() override
	{
		ChildSlot
		[
			WindowContents.ToSharedRef()
		];

		DocumentList->ScrollToTop();
		DocumentTiles->ScrollToTop();
	}

	virtual void RemoveContents() override
	{
		ChildSlot
		[
			SNullWidget::NullWidget
		];
	}

	virtual void ScrollDocumentsToTop() override
	{
		DocumentList->ScrollToTop();
		DocumentTiles->ScrollToTop();
		DocumentScrollBox->ScrollToStart();
	}

	virtual void ScrollDocumentsToBottom() override
	{
		DocumentList->ScrollToBottom();
		DocumentTiles->ScrollToBottom();
		DocumentScrollBox->ScrollToEnd();
	}

private:
	template<typename UserClass>
	TSharedRef<SWidget> ConstructPianoKeyImpl(TSharedRef<UserClass> InUserObjectPtr, FOnClicked::TMethodPtr<UserClass, EPianoKey> InKeyClickHandler, EPianoKey Key)
	{
		TSharedRef<SButton> Button = SNew(SButton)
			.Text(FPianoKeyExtensions::ToText(Key))
			.OnClicked(InUserObjectPtr, InKeyClickHandler, Key)
			.IsEnabled(ViewModel.ToSharedRef(), &IAutomationDriverSpecSuiteViewModel::IsKeyEnabled, Key)
			.OnHovered(ViewModel.ToSharedRef(), &IAutomationDriverSpecSuiteViewModel::KeyHovered, Key)
			.VAlign(VAlign_Bottom)
			.HAlign(HAlign_Center)
			.ContentPadding(FMargin(5, 0, 5, 10))
			.Tag("Key")
			.AddMetaData(FDriverMetaData::Id(*(FString(TEXT("Key")) + FPianoKeyExtensions::ToString(Key))));

		PianoKeys.Add(Key, Button);

		return Button;
	}


private:

	TSharedPtr<IAutomationDriverSpecSuiteViewModel> ViewModel;

	TSharedPtr<SWidget> WindowContents;
	TSharedPtr<SListView<TSharedRef<FDocumentInfo>>> DocumentList;
	TSharedPtr<STileView<TSharedRef<FDocumentInfo>>> DocumentTiles;
	TSharedPtr<SScrollBox> DocumentScrollBox;
	TMap<EPianoKey, TSharedPtr<SButton>> PianoKeys;
	TMap<EPianoKey, TSharedPtr<SMenuAnchor>> PianoKeyMenus;
};

TSharedRef<SAutomationDriverSpecSuite> SAutomationDriverSpecSuite::New()
{
	return MakeShareable(new SAutomationDriverSpecSuiteImpl());
}

