// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/SUserWidget.h"

class IAutomationDriverSpecSuiteViewModel;

enum class EPianoKey : uint8;

class SAutomationDriverSpecSuite 
	: public SUserWidget
{
public:

	SLATE_USER_ARGS(SAutomationDriverSpecSuite)
	{}
	SLATE_END_ARGS()

	virtual void Construct(const FArguments& InArgs, const TSharedRef<IAutomationDriverSpecSuiteViewModel>& InViewModel) = 0;

	virtual TSharedPtr<SWidget> GetKeyWidget(EPianoKey Key) const = 0;

	virtual void RestoreContents() = 0;

	virtual void RemoveContents() = 0;

	virtual void ScrollDocumentsToTop() = 0;

	virtual void ScrollDocumentsToBottom() = 0;
};
